// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Common.ush"
#include "Random.ush"
#include "GammaCorrectionCommon.ush"
#include "/Engine/Shared/VirtualTextureDefinitions.h"
#ifndef VT_DISABLE_VIEW_UNIFORM_BUFFER
#define VT_DISABLE_VIEW_UNIFORM_BUFFER 0
#endif

#ifndef NUM_VIRTUALTEXTURE_SAMPLES
#error NUM_VIRTUALTEXTURE_SAMPLES should be set to the number of VT samples that will be done before including this file (exluding any lightmap samples)
#endif

#ifndef NUM_VIRTUALTEXTURE_FEEDBACK_REQUESTS
#define NUM_VIRTUALTEXTURE_FEEDBACK_REQUESTS NUM_VIRTUALTEXTURE_SAMPLES
#endif

#ifndef LIGHTMAP_VT_ENABLED
#define LIGHTMAP_VT_ENABLED 0
#endif

#ifndef VT_FEEDBACK_RESERVOIR_SAMPLE
#define VT_FEEDBACK_RESERVOIR_SAMPLE 0
#endif

#ifndef VT_SUPPORT_DERIVATIVES
#define VT_SUPPORT_DERIVATIVES (PIXELSHADER || COMPUTESHADER)
#endif

#ifndef VISUALIZE
#define VISUALIZE 0
#endif

/** Struct used to store feedback information in. */
struct FVirtualTextureFeedbackParams
{
	uint Request;		// The stored feedback request for this pixel.
	uint Count;			// Counts the number of calls to StoreVirtualTextureFeedback().
	uint RequestIndex;	// The sample index for the feedback request that we will actually write.
	uint Random;		// A random value in a sequence that we use when reservoir sampling the feedback request to write.
	uint DebugCounts;	// The accumulated debug counts for this pixel from all samples. Stack count in top bits, pending mips in lower bits.
};

/** Initializes the FVirtualTextureFeedbackParams for the pixel shader. */
void InitializeVirtualTextureFeedback(in out FVirtualTextureFeedbackParams Params)
{
	// Init request with "empty" value.
	Params.Request = 0xFFFFFFFF;
	Params.Count = 0;
#if VT_DISABLE_VIEW_UNIFORM_BUFFER
	Params.RequestIndex = 0;
	Params.Random = 0;
#else
	// Init the request index based on the number of feedback requests that we expect to see.
	Params.RequestIndex = View.VirtualTextureFeedbackSampleOffset % max(NUM_VIRTUALTEXTURE_FEEDBACK_REQUESTS + LIGHTMAP_VT_ENABLED, 1);
	Params.Random = View.Random;
#endif
	// Init debug count with number of feedback samples in top 16 bits.
	Params.DebugCounts = (NUM_VIRTUALTEXTURE_SAMPLES + LIGHTMAP_VT_ENABLED) << 16;
}

/** Store feedback info for a VT request in the passed in FVirtualTextureFeedbackParams. */
void StoreVirtualTextureFeedback(in out FVirtualTextureFeedbackParams Params, uint Request, uint PendingMips)
{
#if VT_FEEDBACK_RESERVOIR_SAMPLE
	// Store the feedback request from a single sample selected by reservoir sampling.
	// We select the sample if: rand() % (1 + Count) == 0
	// Or using floating point: frand() <= 1.f / (1.f + Count)
	// We pre-increment and multiply through by Params.Count to avoid a divide.
	// We take frand from upper 24 bits of Params.Random to avoid floating point precision issues.
	// We use a cheap LCG to generate the random value sequence. Offline testing shows this gives sufficient randomness in the final selected sample indices.
	Params.Count++;
	Params.Request = ((float)(Params.Random >> 8) / 16777215.0f) * (float)Params.Count <= 1.0f ? Request : Params.Request;
	Params.Random = 1664525u * Params.Random + 1013904223u;
#elif (NUM_VIRTUALTEXTURE_FEEDBACK_REQUESTS + LIGHTMAP_VT_ENABLED) > 1
	// Only store the feedback request at the requested sample index.
	Params.Request = Params.Count == Params.RequestIndex ? Request : Params.Request;
	Params.Count++;
#else 
	// No more than one feedback request, so always store.
	Params.Request = Request;	
#endif
	Params.DebugCounts += PendingMips;
}

/** This should be called at the end of the pixel shader to write out the gathered VT feedback info to the OutputBuffer. */
void FinalizeVirtualTextureFeedback(in FVirtualTextureFeedbackParams Params, in uint4 FeedbackViewParams, in float4 SvPosition, RWStructuredBuffer<uint> OutputBuffer)
{
	uint2 PixelTilePos = (uint2)SvPosition.xy & FeedbackViewParams.x;
	uint PixelTileIndex = (PixelTilePos.y << FeedbackViewParams.y) + PixelTilePos.x;

	// This code will only run every few pixels...
	[branch] if (PixelTileIndex == FeedbackViewParams.z && Params.Request != 0xFFFFFFFF)
	{
		uint WriteOffset = 0;
		InterlockedAdd(OutputBuffer[0], 1, WriteOffset);
		WriteOffset += 1;

		if (WriteOffset < FeedbackViewParams.w)
		{
			OutputBuffer[WriteOffset] = Params.Request;
		}
	}
}

void FinalizeVirtualTextureFeedback(in FVirtualTextureFeedbackParams Params, float4 SvPosition, RWStructuredBuffer<uint> OutputBuffer, bool bEnableDebug = false)
{
	const uint4 FeedbackViewParams = uint4(View.VirtualTextureFeedbackMask, View.VirtualTextureFeedbackShift, View.VirtualTextureFeedbackJitterOffset, View.VirtualTextureFeedbackBufferSize);
	FinalizeVirtualTextureFeedback(Params, FeedbackViewParams, SvPosition, OutputBuffer);

#if VISUALIZE
	// Write to extended debug buffer if it exists and if we passed in the debug flag.
	// If the debug flag is only driven by a debug permutation then we could potentially drop VISUALIZE defines in this file because the debug logic should just compile out.
	// But we keep the defines to keep things clear and explicit.
	if (bEnableDebug && View.VirtualTextureExtendedDebugBufferSize > 0)
	{
		uint IgnoredOldValue;
		uint WriteOffset = View.VirtualTextureFeedbackBufferSize + (uint)SvPosition.y * (uint)View.ViewSizeAndInvSize.x + (uint)SvPosition.x;
		InterlockedAdd(OutputBuffer[WriteOffset], Params.DebugCounts, IgnoredOldValue);
	}
#endif
}

/**
 * Address Mode Logic
 * Apply before after calculating mip level/derivatives!
 */

#define VTADDRESSMODE_CLAMP 0u
#define VTADDRESSMODE_WRAP 1u
#define VTADDRESSMODE_MIRROR 2u

float ApplyAddressModeMirror(float v)
{
	float t = frac(v * 0.5f) * 2.0f;
	return 1.0f - abs(t - 1.0f);
}

float ApplyAddressMode(float v, uint AddressMode)
{
	// For CLAMP address mode, can't clamp to 1.0f, otherwise 'uint(UV * SizeInPages)' will overflow page table bounds by 1
	// Instead, clamp to slightly below 1, this ensures that when rounded down to uint, above value will be at most 'SizeInPages - 1'
	// The actual texel we clamp to doesn't matter too much for sampling physical texture, since we have borders around the physical pages
	// Just need to make sure we don't clamp too far and chop off valid texels at the edge of texture
	const float MaxTextureSize = 65536.0f;

	if(AddressMode == VTADDRESSMODE_WRAP) return frac(v);
	else if(AddressMode == VTADDRESSMODE_MIRROR) return ApplyAddressModeMirror(v);
	else return clamp(v, 0.0f, 1.0f - (1.0f / MaxTextureSize));
}

float2 ApplyAddressMode(float2 UV, uint AddressU, uint AddressV)
{
	return float2(ApplyAddressMode(UV.x, AddressU), ApplyAddressMode(UV.y, AddressV));
}

/** Non aniso mip level calculation. */
float MipLevel2D( float2 dUVdx, float2 dUVdy )
{
	const float px = dot( dUVdx, dUVdx );
	const float py = dot( dUVdy, dUVdy );
	return 0.5f * log2( max( px, py ) );
}

/** Aniso mip level calculation. */
float MipLevelAniso2D( float2 dUVdx, float2 dUVdy, const float MaxAnisoLog2 )
{
	const float px = dot( dUVdx, dUVdx );
	const float py = dot( dUVdy, dUVdy );

	const float MinLevel = 0.5f * log2( min( px, py ) );
	const float MaxLevel = 0.5f * log2( max( px, py ) );

	const float AnisoBias = min( MaxLevel - MinLevel, MaxAnisoLog2 );
	const float Level = MaxLevel - AnisoBias;
	
	return Level;
}

/** Unpacked contents of per page table uniforms. */
struct VTPageTableUniform
{
	uint XOffsetInPages; // 12
	uint YOffsetInPages; // 12
	uint MaxLevel; // 4
	uint vPageTableMipBias; // 4
	uint MipBiasGroup; // 2
	uint ShiftedPageTableID; // 4
	uint AdaptiveLevelBias; // 4
	uint AllocatedAdaptiveLevelBias; // 4
	bool bPageTable32Bit;

	float2 SizeInPages;
	float2 UVScale;
	float MaxAnisoLog2;
};

/** Unpack the per page table uniforms. */
VTPageTableUniform VTPageTableUniform_Unpack(uint4 PackedPageTableUniform0, uint4 PackedPageTableUniform1)
{
	VTPageTableUniform result;
	result.UVScale = asfloat(PackedPageTableUniform0.xy);
	result.SizeInPages = asfloat(PackedPageTableUniform0.zw);
	result.MaxAnisoLog2 = asfloat(PackedPageTableUniform1.x);
	result.XOffsetInPages = PackedPageTableUniform1.y & 0xfff;
	result.YOffsetInPages = (PackedPageTableUniform1.y >> 12) & 0xfff;
	result.vPageTableMipBias = (PackedPageTableUniform1.y >> 24) & 0xf;
	result.MipBiasGroup = (PackedPageTableUniform1.y >> 28) & 0x3;
	result.MaxLevel = PackedPageTableUniform1.z & 0xf;
	result.AdaptiveLevelBias = (PackedPageTableUniform1.z >> 4) & 0xf;
	result.AllocatedAdaptiveLevelBias = (PackedPageTableUniform1.z >> 9) & 0xf;
	result.bPageTable32Bit = (PackedPageTableUniform1.z & 0x100) != 0;
	result.ShiftedPageTableID = PackedPageTableUniform1.w;
	return result;
}

VTPageTableUniform VTPageTableUniform_Unpack(uint2 PackedPageTableUniform)
{
	VTPageTableUniform Result;
	Result.UVScale = 1.f;
	Result.SizeInPages.x = PackedPageTableUniform.y & 0xfff;
	Result.SizeInPages.y = (PackedPageTableUniform.y >> 12) & 0xfff;
	Result.MaxAnisoLog2 = 0;
	Result.XOffsetInPages = PackedPageTableUniform.x & 0xfff;
	Result.YOffsetInPages = (PackedPageTableUniform.x >> 12) & 0xfff;
	Result.vPageTableMipBias = (PackedPageTableUniform.x >> 24) & 0xf;
	Result.MipBiasGroup = 0;
	Result.MaxLevel = (PackedPageTableUniform.y >> 24) & 0xf;
	Result.AdaptiveLevelBias = 0;
	Result.AllocatedAdaptiveLevelBias = 0;
	Result.bPageTable32Bit = (PackedPageTableUniform.y >> 28) & 0x1;
	Result.ShiftedPageTableID = PackedPageTableUniform.x & 0xf0000000;
	return Result;
}

bool IsValid(VTPageTableUniform PageTableUniform)
{
	// This is expected to happen when we have null MeshPaintTextureDescriptor values.
	// In that case we want to skip page table sampling and use the FallbackValue.
	return PageTableUniform.SizeInPages.x > 0;
}

/** Structure carrying page table read result. Is passed as inout parameter and partially filled by several functions. */
struct VTPageTableResult
{
	float2 UV;
	float2 dUVdx;
	float2 dUVdy;
	uint4 PageTableValue[4];
	uint PackedRequest;
	float MipLevelFrac;
	bool bPageTable32Bit;
	uint PendingMips;
	uint AllocatedAdaptiveBias;
};

uint GetVirtualTexturePendingMips(uint RequestedLevel, uint4 PackedPageTableValue, uint AccumulatedResult)
{
#if VISUALIZE
	// 0 is an empty value in the PageTableResult.
	// We must also ignore a value of 1 for this test since it will be in the .w channel for page table textures with less than 4 channels.
	// So only consider packed page table values > 1.
	uint4 SampledLevels = select(PackedPageTableValue.xyzw > 1,  PackedPageTableValue.xyzw & 0xf, ~0u);
	uint MinSampledLevel = min(min(min(SampledLevels.x, SampledLevels.y), SampledLevels.z), SampledLevels.w);
	if (MinSampledLevel != ~0u && MinSampledLevel > RequestedLevel)
	{
		AccumulatedResult = max(MinSampledLevel - RequestedLevel, AccumulatedResult);
	}
#endif

	return AccumulatedResult;
}

float GetStochasticMipNoise(float2 SvPositionXY)
{
#if VT_DISABLE_VIEW_UNIFORM_BUFFER
	return 0;
#else
	return InterleavedGradientNoise(SvPositionXY, View.StateFrameIndexMod8);
#endif
}

float GetGlobalVirtualTextureMipBias(int Group)
{
#if VT_DISABLE_VIEW_UNIFORM_BUFFER
	return 0;
#else
	return View.GlobalVirtualTextureMipBias[Group];
#endif
}

/** Calculate mip level including stochastic noise. Also stores derivatives to OutResult for use when sampling physical texture. */
int TextureComputeVirtualMipLevel(
	in out VTPageTableResult OutResult,
	float2 dUVdx, float2 dUVdy, float MipBias,
	float2 SvPositionXY,
	VTPageTableUniform PageTableUniform)
{
	OutResult.dUVdx = dUVdx * PageTableUniform.SizeInPages;
	OutResult.dUVdy = dUVdy * PageTableUniform.SizeInPages;

	// Always compute mip level using MipLevelAniso2D, even if VIRTUAL_TEXTURE_ANISOTROPIC_FILTERING is disabled
	// This way the VT mip level selection will come much closer to HW mip selection, even if we're not sampling the texture using anisotropic filtering
	const float ComputedLevel = MipLevelAniso2D(OutResult.dUVdx, OutResult.dUVdy, PageTableUniform.MaxAnisoLog2);

	const float GlobalMipBias = GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup);
#if VIRTUAL_TEXTURE_MANUAL_TRILINEAR_FILTERING
	const float Noise = 0.f;
#elif VIRTUAL_TEXTURE_FORCE_BILINEAR_FILTERING
	const float Noise = 0.5f;
#else
	const float Noise = GetStochasticMipNoise(SvPositionXY);
#endif

	const float MipLevel = ComputedLevel + MipBias + GlobalMipBias + Noise;
	const float MipLevelFloor = floor(MipLevel);
	
	OutResult.MipLevelFrac = MipLevel - MipLevelFloor;

	return (int)MipLevelFloor + int(PageTableUniform.vPageTableMipBias);
}

/** Samples page table indirection and any apply changes to UV, vLevel and PageTable information for adaptive page tables. */
void ApplyAdaptivePageTableUniform(
	Texture2D<uint> PageTableIndirection,
	in out VTPageTableResult InOutResult,
	in out VTPageTableUniform InOutPageTableUniform,
	in out float2 UV,
	in out int vLevel)
{
	if (vLevel < 0)
	{
		// Requested level not stored in low mips so find the adaptive page table entry for this UV.
		float2 AdaptiveGridPos = UV * InOutPageTableUniform.SizeInPages;
		int2 AdaptiveGridCoord = (int2)floor(AdaptiveGridPos);
		float2 AdaptiveGridUV = frac(AdaptiveGridPos);
		uint PackedAdaptiveDesc = PageTableIndirection.Load(int3(AdaptiveGridCoord, 0));

		[branch]
		if (PackedAdaptiveDesc != 0)
		{
			// A valid entry was found so apply changes.
			InOutPageTableUniform.XOffsetInPages = PackedAdaptiveDesc & 0xfff;
			InOutPageTableUniform.YOffsetInPages = (PackedAdaptiveDesc >> 12) & 0xfff;
			InOutPageTableUniform.MaxLevel = (PackedAdaptiveDesc >> 24) & 0xf;
			InOutPageTableUniform.SizeInPages = ((int) 1) << InOutPageTableUniform.MaxLevel;

			vLevel += InOutPageTableUniform.MaxLevel;
			InOutResult.dUVdx *= InOutPageTableUniform.SizeInPages;
			InOutResult.dUVdy *= InOutPageTableUniform.SizeInPages;

			UV = frac(AdaptiveGridPos);
		}
	}
}

/** Load from page table and store results. Single page table texture version. */
void TextureLoadVirtualPageTableInternal(
	in out VTPageTableResult OutResult,
	Texture2D<uint4> PageTable0,
	VTPageTableUniform PageTableUniform,
	float2 UV, int vLevel)
{
	OutResult.UV = UV * PageTableUniform.SizeInPages;
	OutResult.bPageTable32Bit = PageTableUniform.bPageTable32Bit;
	OutResult.AllocatedAdaptiveBias = PageTableUniform.AllocatedAdaptiveLevelBias;

	const uint vLevelClamped = clamp(vLevel, 0, int(PageTableUniform.MaxLevel));
	uint vPageX = (uint(OutResult.UV.x) + PageTableUniform.XOffsetInPages) >> vLevelClamped;
	uint vPageY = (uint(OutResult.UV.y) + PageTableUniform.YOffsetInPages) >> vLevelClamped;

	OutResult.PageTableValue[0] = PageTable0.Load(int3(vPageX, vPageY, vLevelClamped));
	OutResult.PageTableValue[1] = uint4(0u, 0u, 0u, 0u);

	OutResult.PendingMips = GetVirtualTexturePendingMips(vLevelClamped, OutResult.PageTableValue[0], 0);

#if VIRTUAL_TEXTURE_MANUAL_TRILINEAR_FILTERING
	// Second page table for trilinear.
	const uint vLevelClamped2 = clamp(vLevel + 1, 0, int(PageTableUniform.MaxLevel));
	const uint vPageX2 = (uint(OutResult.UV.x) + PageTableUniform.XOffsetInPages) >> vLevelClamped2;
	const uint vPageY2 = (uint(OutResult.UV.y) + PageTableUniform.YOffsetInPages) >> vLevelClamped2;
	
	OutResult.PageTableValue[2] = PageTable0.Load(int3(vPageX2, vPageY2, vLevelClamped2));
	OutResult.PageTableValue[3] = uint4(0u, 0u, 0u, 0u);
	
	OutResult.PendingMips = GetVirtualTexturePendingMips(vLevelClamped2, OutResult.PageTableValue[2], OutResult.PendingMips);

	// Alternate requests to both mip levels
	if ((View.FrameNumber & 1u) == 0u)
	{
		vLevel += 1;
		vPageX = vPageX2;
		vPageY = vPageY2;
	}
#endif

	// PageTableID packed in upper 4 bits of 'PackedPageTableUniform', which is the bit position we want it in for PackedRequest as well, just need to mask off extra bits
	OutResult.PackedRequest = PageTableUniform.ShiftedPageTableID;
	OutResult.PackedRequest |= vPageX;
	OutResult.PackedRequest |= vPageY << 12;

	// Feedback always encodes vLevel+N, and subtracts N on the CPU side.
	// This allows the CPU code to know when we requested a negative vLevel which indicates that we don't have sufficient virtual texture resolution.
	const uint vFeedbackLevel = clamp(vLevel + VirtualTextureFeedbackBias, 0, int(PageTableUniform.MaxLevel + VirtualTextureFeedbackBias));
	OutResult.PackedRequest |= vFeedbackLevel << 24;
}

/** Load from page table and store results. Two page table texture version. */
void TextureLoadVirtualPageTableInternal(
	in out VTPageTableResult OutResult,
	Texture2D<uint4> PageTable0, Texture2D<uint4> PageTable1,
	VTPageTableUniform PageTableUniform,
	float2 UV, int vLevel)
{
	OutResult.UV = UV * PageTableUniform.SizeInPages;
	OutResult.bPageTable32Bit = PageTableUniform.bPageTable32Bit;
	OutResult.AllocatedAdaptiveBias = PageTableUniform.AllocatedAdaptiveLevelBias;

	const uint vLevelClamped = clamp(vLevel, 0, int(PageTableUniform.MaxLevel));
	uint vPageX = (uint(OutResult.UV.x) + PageTableUniform.XOffsetInPages) >> vLevelClamped;
	uint vPageY = (uint(OutResult.UV.y) + PageTableUniform.YOffsetInPages) >> vLevelClamped;

	OutResult.PageTableValue[0] = PageTable0.Load(int3(vPageX, vPageY, vLevelClamped));
	OutResult.PageTableValue[1] = PageTable1.Load(int3(vPageX, vPageY, vLevelClamped));

	OutResult.PendingMips = GetVirtualTexturePendingMips(vLevelClamped, OutResult.PageTableValue[0], 0);
	OutResult.PendingMips = GetVirtualTexturePendingMips(vLevelClamped, OutResult.PageTableValue[1], OutResult.PendingMips);

#if VIRTUAL_TEXTURE_MANUAL_TRILINEAR_FILTERING
	// Second page table for trilinear.
	const uint vLevelClamped2 = clamp(vLevel + 1, 0, int(PageTableUniform.MaxLevel));
	const uint vPageX2 = (uint(OutResult.UV.x) + PageTableUniform.XOffsetInPages) >> vLevelClamped2;
	const uint vPageY2 = (uint(OutResult.UV.y) + PageTableUniform.YOffsetInPages) >> vLevelClamped2;

	OutResult.PageTableValue[2] = PageTable0.Load(int3(vPageX2, vPageY2, vLevelClamped2));
	OutResult.PageTableValue[3] = PageTable1.Load(int3(vPageX2, vPageY2, vLevelClamped2));

	// Alternate requests to both mip levels
	if ((View.FrameNumber & 1u) == 0u)
	{
		vLevel += 1;
		vPageX = vPageX2;
		vPageY = vPageY2;
	}
#endif

	// PageTableID packed in upper 4 bits of 'PackedPageTableUniform', which is the bit position we want it in for PackedRequest as well, just need to mask off extra bits
	OutResult.PackedRequest = PageTableUniform.ShiftedPageTableID;
	OutResult.PackedRequest |= vPageX;
	OutResult.PackedRequest |= vPageY << 12;

	// Feedback always encodes vLevel+N, and subtracts N on the CPU side.
	// This allows the CPU code to know when we requested a negative vLevel which indicates that we don't have sufficient virtual texture resolution.
	const uint vFeedbackLevel = clamp(vLevel + VirtualTextureFeedbackBias, 0, int(PageTableUniform.MaxLevel + VirtualTextureFeedbackBias));
	OutResult.PackedRequest |= vFeedbackLevel << 24;
}

/**
 * Public functions used by the material system to sample virtual textures.
 * These boiler plate functions implement the used permutations from the matrix of sampling behaviours:
 *  - One or two page table textures
 *  - Sample(Bias)/SampleGrad/SampleLevel
 *  - Adaptive page table indirection on or off
 *  - Feedback on or off
 *  - Anisotropic filtering on or off
 */ 

// LoadPageTable: 1 Page table
VTPageTableResult TextureLoadVirtualPageTable(
	Texture2D<uint4> PageTable0,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV, 
	MaterialFloat MipBias, float2 SvPositionXY,
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		UV = UV * PageTableUniform.UVScale;
		int vLevel = GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup);
#if VT_SUPPORT_DERIVATIVES
		vLevel = TextureComputeVirtualMipLevel(Result, DDX(UV), DDY(UV), MipBias, SvPositionXY, PageTableUniform);
#endif // VT_SUPPORT_DERIVATIVES
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
		StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	}
	return Result;
}

// LoadPageTable: 1 Page table, No feedback
VTPageTableResult TextureLoadVirtualPageTable(
	Texture2D<uint4> PageTable0,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV, 
	MaterialFloat MipBias, float2 SvPositionXY)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		UV = UV * PageTableUniform.UVScale;
		int vLevel = GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup);
#if VT_SUPPORT_DERIVATIVES
		vLevel = TextureComputeVirtualMipLevel(Result, DDX(UV), DDY(UV), MipBias, SvPositionXY, PageTableUniform);
#endif // VT_SUPPORT_DERIVATIVES
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
	}
	return Result;
}

// LoadPageTable: 2 Page tables
VTPageTableResult TextureLoadVirtualPageTable(
	Texture2D<uint4> PageTable0, Texture2D<uint4> PageTable1,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat MipBias, float2 SvPositionXY,
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		UV = UV * PageTableUniform.UVScale;
		int vLevel = GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup);
#if VT_SUPPORT_DERIVATIVES
		vLevel = TextureComputeVirtualMipLevel(Result, DDX(UV), DDY(UV), MipBias, SvPositionXY, PageTableUniform);
#endif // VT_SUPPORT_DERIVATIVES
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTable1, PageTableUniform, UV, vLevel);
		StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	}
	return Result;
}

// LoadPageTable: 2 Page tables, No feedback
VTPageTableResult TextureLoadVirtualPageTable(
	Texture2D<uint4> PageTable0, Texture2D<uint4> PageTable1,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat MipBias, float2 SvPositionXY)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		UV = UV * PageTableUniform.UVScale;
		int vLevel = GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup);
#if VT_SUPPORT_DERIVATIVES
		vLevel = TextureComputeVirtualMipLevel(Result, DDX(UV), DDY(UV), MipBias, SvPositionXY, PageTableUniform);
#endif // VT_SUPPORT_DERIVATIVES
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTable1, PageTableUniform, UV, vLevel);
	}
	return Result;
}

// LoadPageTable: 1 Page table, SampleGrad
VTPageTableResult TextureLoadVirtualPageTableGrad(
	Texture2D<uint4> PageTable0,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat2 dUVdx, MaterialFloat2 dUVdy, float2 SvPositionXY,
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		int vLevel = TextureComputeVirtualMipLevel(Result, dUVdx * PageTableUniform.UVScale, dUVdy * PageTableUniform.UVScale, 0, SvPositionXY, PageTableUniform);
		UV = UV * PageTableUniform.UVScale;
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
		StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	}
	return Result;
}

// LoadPageTable: 1 Page table, SampleGrad, no feedback
VTPageTableResult TextureLoadVirtualPageTableGrad(
	Texture2D<uint4> PageTable0,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat2 dUVdx, MaterialFloat2 dUVdy, float2 SvPositionXY)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		int vLevel = TextureComputeVirtualMipLevel(Result, dUVdx * PageTableUniform.UVScale, dUVdy * PageTableUniform.UVScale, 0, SvPositionXY, PageTableUniform);
		UV = UV * PageTableUniform.UVScale;
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
	}
	return Result;
}

// LoadPageTable: 2 Page tables, SampleGrad
VTPageTableResult TextureLoadVirtualPageTableGrad(
	Texture2D<uint4> PageTable0, Texture2D<uint4> PageTable1,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat2 dUVdx, MaterialFloat2 dUVdy, float2 SvPositionXY,
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		int vLevel = TextureComputeVirtualMipLevel(Result, dUVdx * PageTableUniform.UVScale, dUVdy * PageTableUniform.UVScale, 0, SvPositionXY, PageTableUniform);
		UV = UV * PageTableUniform.UVScale;
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTable1, PageTableUniform, UV, vLevel);
		StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	}
	return Result;
}

// LoadPageTable: 2 Page tables, SampleGrad, no feedback
VTPageTableResult TextureLoadVirtualPageTableGrad(
	Texture2D<uint4> PageTable0, Texture2D<uint4> PageTable1,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat2 dUVdx, MaterialFloat2 dUVdy, float2 SvPositionXY)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		int vLevel = TextureComputeVirtualMipLevel(Result, dUVdx * PageTableUniform.UVScale, dUVdy * PageTableUniform.UVScale, 0, SvPositionXY, PageTableUniform);
		UV = UV * PageTableUniform.UVScale;
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTable1, PageTableUniform, UV, vLevel);
	}
	return Result;
}

// LoadPageTable: 1 Page table, SampleLevel
VTPageTableResult TextureLoadVirtualPageTableLevel(
	Texture2D<uint4> PageTable0,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat Level, 
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		UV = UV * PageTableUniform.UVScale;
		int vLevel = (int)floor(Level + GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup));
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
		StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	}
	return Result;
}

// LoadPageTable: 1 Page table, SampleLevel, No feedback
VTPageTableResult TextureLoadVirtualPageTableLevel(
	Texture2D<uint4> PageTable0,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat Level)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		UV = UV * PageTableUniform.UVScale;
		int vLevel = (int)floor(Level + GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup));
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
	}
	return Result;
}

// LoadPageTable: 2 Page tables, SampleLevel
VTPageTableResult TextureLoadVirtualPageTableLevel(
	Texture2D<uint4> PageTable0, Texture2D<uint4> PageTable1,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat Level,
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	if (IsValid(PageTableUniform))
	{
		UV = UV * PageTableUniform.UVScale;
		int vLevel = (int)floor(Level + GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup));
		UV = ApplyAddressMode(UV, AddressU, AddressV);
		TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTable1, PageTableUniform, UV, vLevel);
		StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	}
	return Result;
}

// LoadPageTable: 2 Page tables, SampleLevel, No feedback
VTPageTableResult TextureLoadVirtualPageTableLevel(
	Texture2D<uint4> PageTable0, Texture2D<uint4> PageTable1,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat Level)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	UV = UV * PageTableUniform.UVScale;
	int vLevel = (int)floor(Level + GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup));
	UV = ApplyAddressMode(UV, AddressU, AddressV);
	TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTable1, PageTableUniform, UV, vLevel);
	return Result;
}

// LoadPageTable: 1 Page table, Adaptive
VTPageTableResult TextureLoadVirtualPageTableAdaptive(
	Texture2D<uint4> PageTable0,
	Texture2D<uint> PageTableIndirection,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat MipBias, float2 SvPositionXY,
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	UV = UV * PageTableUniform.UVScale;
	int vLevel = GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup);
#if VT_SUPPORT_DERIVATIVES
	vLevel = TextureComputeVirtualMipLevel(Result, DDX(UV), DDY(UV), MipBias, SvPositionXY, PageTableUniform);
#endif // VT_SUPPORT_DERIVATIVES
	UV = ApplyAddressMode(UV, AddressU, AddressV);
	ApplyAdaptivePageTableUniform(PageTableIndirection, Result, PageTableUniform, UV, vLevel);
	TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
	StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	return Result;
}

// LoadPageTable: 1 Page table, Adaptive, SampleGrad
VTPageTableResult TextureLoadVirtualPageTableAdaptiveGrad(
	Texture2D<uint4> PageTable0,
	Texture2D<uint> PageTableIndirection,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat2 dUVdx, MaterialFloat2 dUVdy, float2 SvPositionXY,
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	int vLevel = TextureComputeVirtualMipLevel(Result, dUVdx * PageTableUniform.UVScale, dUVdy * PageTableUniform.UVScale, 0, SvPositionXY, PageTableUniform);
	UV = UV * PageTableUniform.UVScale;
	UV = ApplyAddressMode(UV, AddressU, AddressV);
	ApplyAdaptivePageTableUniform(PageTableIndirection, Result, PageTableUniform, UV, vLevel);
	TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
	StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	return Result;
}

// LoadPageTable: 1 Page table, Adaptive, SampleLevel
VTPageTableResult TextureLoadVirtualPageTableAdaptiveLevel(
	Texture2D<uint4> PageTable0,
	Texture2D<uint> PageTableIndirection,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat Level, 
	in out FVirtualTextureFeedbackParams Feedback)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	// Level is an index into the full size adaptive VT. AdaptiveLevelBias shifts it relative to the low mips allocated VT.
	int vLevel = (int)floor(Level + GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup)) - PageTableUniform.AdaptiveLevelBias;
	UV = UV * PageTableUniform.UVScale;
	UV = ApplyAddressMode(UV, AddressU, AddressV);
	ApplyAdaptivePageTableUniform(PageTableIndirection, Result, PageTableUniform, UV, vLevel);
	TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
	StoreVirtualTextureFeedback(Feedback, Result.PackedRequest, Result.PendingMips);
	return Result;
}

// LoadPageTable: 1 Page table, Adaptive, SampleLevel, No feedback
VTPageTableResult TextureLoadVirtualPageTableAdaptiveLevel(
	Texture2D<uint4> PageTable0,
	Texture2D<uint> PageTableIndirection,
	VTPageTableUniform PageTableUniform,
	float2 UV, uint AddressU, uint AddressV,
	MaterialFloat Level)
{
	VTPageTableResult Result = (VTPageTableResult)0;
	// Level is an index into the full size adaptive VT. AdaptiveLevelBias shifts it relative to the low mips allocated VT.
	int vLevel = (int)floor(Level + GetGlobalVirtualTextureMipBias(PageTableUniform.MipBiasGroup)) - PageTableUniform.AdaptiveLevelBias;
	UV = UV * PageTableUniform.UVScale;
	UV = ApplyAddressMode(UV, AddressU, AddressV);
	ApplyAdaptivePageTableUniform(PageTableIndirection, Result, PageTableUniform, UV, vLevel);
	TextureLoadVirtualPageTableInternal(Result, PageTable0, PageTableUniform, UV, vLevel);
	return Result;
}

/** Unpacked contents of per physical sample uniform. */
struct VTUniform
{
	// Page sizes are scaled by RcpPhysicalTextureSize
	float pPageSize;
	float vPageSize;
	float vPageBorderSize;
	float4 FallbackValue;
};

/** Unpack the physical sample uniform. */
VTUniform VTUniform_Unpack(uint4 PackedUniform)
{
	VTUniform result;
	result.pPageSize = asfloat(PackedUniform.w);
	result.vPageSize = asfloat(PackedUniform.y);
	result.vPageBorderSize = asfloat(PackedUniform.z);
	result.FallbackValue.b = float((PackedUniform.x >> 0) & 0xFF) * (1.0f / 255.0f);
	result.FallbackValue.g = float((PackedUniform.x >> 8) & 0xFF) * (1.0f / 255.0f);
	result.FallbackValue.r = float((PackedUniform.x >> 16) & 0xFF) * (1.0f / 255.0f);
	result.FallbackValue.a = float((PackedUniform.x >> 24) & 0xFF) * (1.0f / 255.0f);
	return result;
}

/** We use 0 to mark an unmapped page table entry. */
bool IsValid(VTPageTableResult PageTableResult, uint LayerIndex)
{
	const uint PackedPageTableValue = PageTableResult.PageTableValue[LayerIndex / 4u][LayerIndex & 3u];
	return (PackedPageTableValue >> 4) != 0;
}

/** Applies proper scaling to dUVdx/dUVdy in PageTableResult. */
float2 VTComputePhysicalUVs(in out VTPageTableResult PageTableResult, uint LayerIndex, VTUniform Uniform)
{
	const uint PackedPageTableValue = PageTableResult.PageTableValue[LayerIndex / 4u][LayerIndex & 3u];

	// See packing in PageTableUpdate.usf
	const uint vLevel = (PackedPageTableValue & 0xf) - PageTableResult.AllocatedAdaptiveBias;
	const float UVScale = float(4096u >> vLevel) * (1.0f / 4096.0f);

	// This will compile to runtime branch, but should in theory be conditional moves selecting 1 of 2 different bitfield extracting instructions
	const uint pPageX = PageTableResult.bPageTable32Bit ? (PackedPageTableValue >> 4) & 0xff : (PackedPageTableValue >> 4) & 0x3f;
	const uint pPageY = PageTableResult.bPageTable32Bit ? (PackedPageTableValue >> 12) & 0xff : (PackedPageTableValue >> 10) & 0x3f;

	const float2 vPageFrac = frac(PageTableResult.UV * UVScale);
	const float2 pUV = float2(pPageX, pPageY) * Uniform.pPageSize + (vPageFrac * Uniform.vPageSize + Uniform.vPageBorderSize);

	const float ddxyScale = UVScale * Uniform.vPageSize;
	PageTableResult.dUVdx *= ddxyScale;
	PageTableResult.dUVdy *= ddxyScale;
	return pUV;
}


MaterialFloat4 TextureVirtualSampleAniso(Texture2D Physical, SamplerState PhysicalSampler, float2 pUV, 	float2 dUVdx, float2 dUVdy)
{
#if	VIRTUAL_TEXTURE_ANISOTROPIC_FILTERING
	return Physical.SampleGrad(PhysicalSampler, pUV, dUVdx, dUVdy);
#else
	// no need for dUVdx/dUVdy unless we have anistropic filtering enabled
	return Physical.SampleLevel(PhysicalSampler, pUV, 0.0f);
#endif
}

// SamplePhysicalTexture: Aniso On
MaterialFloat4 TextureVirtualSample(
	Texture2D Physical, SamplerState PhysicalSampler,
	VTPageTableResult PageTableResult, uint LayerIndex,
	VTUniform Uniform)
{
	bool bValid = IsValid(PageTableResult, LayerIndex);
	
	MaterialFloat4 Sample1;
	if (bValid)
	{
		float2 pUV = VTComputePhysicalUVs(PageTableResult, LayerIndex, Uniform);
		Sample1 = TextureVirtualSampleAniso(Physical, PhysicalSampler, pUV, PageTableResult.dUVdx, PageTableResult.dUVdy);
	}
	else
	{
		Sample1 = Uniform.FallbackValue;
	}

#if VIRTUAL_TEXTURE_MANUAL_TRILINEAR_FILTERING
	bValid = IsValid(PageTableResult, LayerIndex + 8u);
	MaterialFloat4 Sample2;
	if (bValid)
	{
		float2 pUV = VTComputePhysicalUVs(PageTableResult, LayerIndex + 8u, Uniform);
		Sample2 = TextureVirtualSampleAniso(Physical, PhysicalSampler, pUV, PageTableResult.dUVdx, PageTableResult.dUVdy);
	}
	else
	{
		Sample2 = Sample1;
	}
	return lerp(Sample1, Sample2, PageTableResult.MipLevelFrac);
#else
	return Sample1;
#endif
}

// SamplePhysicalTexture: Aniso Off
MaterialFloat4 TextureVirtualSampleLevel(
	Texture2D Physical, SamplerState PhysicalSampler,
	VTPageTableResult PageTableResult, uint LayerIndex,
	VTUniform Uniform)
{
	const bool bValid = IsValid(PageTableResult, LayerIndex);
	if (bValid)
	{
		const float2 pUV = VTComputePhysicalUVs(PageTableResult, LayerIndex, Uniform);
		// No need to apply dUVdx/dUVdy, don't support anisotropic when sampling a specific level
		return Physical.SampleLevel(PhysicalSampler, pUV, 0.0f);
	}
	return Uniform.FallbackValue;
}


/**	Helper function to convert world space to virtual texture UV space. */
float2 VirtualTextureWorldToUV(in FDFVector3 WorldPos, in FDFVector3 O, in float3 U, in float3 V)
{
	// After the subtract we can assume that the offset is small enough for float maths.
	float3 P = DFFastSubtractDemote(WorldPos, O);
	return float2(dot(P, U), dot(P, V));
}

/**	Helper function to convert world space to virtual texture UV space. */
float2 VirtualTextureWorldToUV(in FLWCVector3 WorldPos, in FLWCVector3 O, in float3 U, in float3 V)
{
	// After the subtract we can assume that the offset is small enough for float maths.
	float3 P = LWCToFloat(LWCSubtract(WorldPos, O));
	return float2(dot(P, U), dot(P, V));
}

/**	Helper function to convert translated world space to virtual texture UV space (requires O to be in translated space). */
float2 VirtualTextureWorldToUV(in float3 TranslatedWorldPos, in float3 O, in float3 U, in float3 V)
{
	float3 P = TranslatedWorldPos - O;
	return float2(dot(P, U), dot(P, V));
}

/**	Derivative-aware helper function to convert world space to virtual texture UV space. */
FloatDeriv2 VirtualTextureWorldToUVDeriv(in FDFVector3Deriv WorldPos, in FDFVector3 O, in float3 U, in float3 V)
{
	FloatDeriv2 Result;
	Result.Value = VirtualTextureWorldToUV(WorldPos.Value, O, U, V);
	Result.Ddx = float2(dot(WorldPos.Ddx, U), dot(WorldPos.Ddx, V));
	Result.Ddy = float2(dot(WorldPos.Ddy, U), dot(WorldPos.Ddy, V));
	return Result;
}

/**	Derivative-aware helper function to convert world space to virtual texture UV space. */
FloatDeriv2 VirtualTextureWorldToUVDeriv(in FLWCVector3Deriv WorldPos, in FLWCVector3 O, in float3 U, in float3 V)
{
	FloatDeriv2 Result;
	Result.Value = VirtualTextureWorldToUV(WorldPos.Value, O, U, V);
	Result.Ddx = float2(dot(WorldPos.Ddx, U), dot(WorldPos.Ddx, V));
	Result.Ddy = float2(dot(WorldPos.Ddy, U), dot(WorldPos.Ddy, V));
	return Result;
}

/**	Derivative-aware helper function to convert translated world space to virtual texture UV space (requires O to be in translated space). */
FloatDeriv2 VirtualTextureWorldToUVDeriv(in FloatDeriv3 WorldPos, in float3 O, in float3 U, in float3 V)
{
	FloatDeriv2 Result;
	Result.Value = VirtualTextureWorldToUV(WorldPos.Value, O, U, V);
	Result.Ddx = float2(dot(WorldPos.Ddx, U), dot(WorldPos.Ddx, V));
	Result.Ddy = float2(dot(WorldPos.Ddy, U), dot(WorldPos.Ddy, V));
	return Result;
}

/** Unpack color from YCoCg stored in BC3. */
float3 VirtualTextureUnpackBaseColorYCoCg(in float4 PackedValue)
{
	float Y = PackedValue.a;
	float Scale = 1.f / ((255.f / 8.f) * PackedValue.b + 1.f);
	float Co = (PackedValue.r - 128.f / 255.f) * Scale;
	float Cg = (PackedValue.g - 128.f / 255.f) * Scale;
	return float3(Y + Co - Cg, Y + Cg, Y - Co - Cg);
}

/** Generic normal unpack funtion. */
float3 VirtualTextureUnpackNormal(in float2 PackedXY, in float PackedSignZ)
{
	float2 NormalXY = PackedXY * (255.f / 127.f) - 1.f;
	float SignZ = PackedSignZ * 2.f - 1.f;
	float NormalZ = sqrt(saturate(1.0f - dot(NormalXY, NormalXY))) * SignZ;
	return float3(NormalXY, NormalZ);
}

/** Unpack normal from BC3. */
float3 VirtualTextureUnpackNormalBC3(in float4 PackedValue)
{
	return VirtualTextureUnpackNormal(PackedValue.ag, 1.f);
}

/** Unpack normal from two BC3 textures. */
float3 VirtualTextureUnpackNormalBC3BC3(in float4 PackedValue0, in float4 PackedValue1)
{
	return VirtualTextureUnpackNormal(float2(PackedValue0.a, PackedValue1.a), PackedValue1.b);
}

/** Unpack normal from BC5. */
float3 VirtualTextureUnpackNormalBC5(in float4 PackedValue)
{
	return VirtualTextureUnpackNormal(PackedValue.rg, 1.f);
}

/** Unpack normal from BC5 and BC1 textures. */
float3 VirtualTextureUnpackNormalBC5BC1(in float4 PackedValue0, in float4 PackedValue1)
{
	return VirtualTextureUnpackNormal(float2(PackedValue0.x, PackedValue0.y), PackedValue1.b);
}

/** Unpack 16 bit height (with some hardcoded range scale/bias). */
float VirtualTextureUnpackHeight(in float4 PackedValue, in float2 UnpackHeightScaleBias)
{
	return PackedValue.r * UnpackHeightScaleBias.x + UnpackHeightScaleBias.y;
}


float3 VirtualTextureUnpackNormalBGR565(in float4 PackedValue)
{
	// sign of normal z is considered always positive to save channels
	return VirtualTextureUnpackNormal(PackedValue.xz, 1.0);
}

/** Unpack SRGB Color */
float3 VirtualTextureUnpackBaseColorSRGB(in float4 PackedValue)
{
	return sRGBToLinear(PackedValue.rgb);
}
