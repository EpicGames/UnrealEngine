// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VirtualTextureCompress.usf: 
	Compression pass for runtime virtual texture
=============================================================================*/

#include "Common.ush"
#include "BlockCompressionCommon.ush"
#include "BCCompressionCommon.ush"
#include "ASTCCompressionCommon.ush"
#include "ETCCompressionCommon.ush"

#ifndef ETC_PROFILE
#define ETC_PROFILE 0
#endif

#ifndef ASTC_PROFILE
#define ASTC_PROFILE 0
#endif

#ifndef PACK_RG32_RGBA16
#define PACK_RG32_RGBA16 0
#endif

#if PACK_RG32_RGBA16
#define TYPE_64BIT uint4
#else
#define TYPE_64BIT uint2
#endif

#ifndef USE_SRC_TEXTURE_ARRAY
#define USE_SRC_TEXTURE_ARRAY 0
#endif

#ifndef USE_DST_TEXTURE_ARRAY
#define USE_DST_TEXTURE_ARRAY 0
#endif

#if USE_SRC_TEXTURE_ARRAY
#define SourceTextureType Texture2DArray
#else
#define SourceTextureType Texture2D
#endif

#if USE_DST_TEXTURE_ARRAY
#define DestTextureType RWTexture2DArray
#else
#define DestTextureType RWTexture2D
#endif

SourceTextureType<float4> RenderTexture0;
SamplerState TextureSampler0;
SourceTextureType<float4> RenderTexture1;
SamplerState TextureSampler1;
SourceTextureType<float4> RenderTexture2;
SamplerState TextureSampler2;

// Outputs can be 64bit or 128 bit depending on formats.
// The 64 bit type can be uint2 or uint4 depending on platform support.
DestTextureType<TYPE_64BIT> OutCompressTexture0_64bit;
DestTextureType<TYPE_64BIT> OutCompressTexture1_64bit;
DestTextureType<TYPE_64BIT> OutCompressTexture2_64bit;
DestTextureType<uint4> OutCompressTexture0_128bit;
DestTextureType<uint4> OutCompressTexture1_128bit;
DestTextureType<uint4> OutCompressTexture2_128bit;

uint4 SourceRect;
DECLARE_SCALAR_ARRAY(uint, DestPos, MAX_BATCH_SIZE * MAX_DST_LAYERS * 2);

#if PACK_RG32_RGBA16
/** Pack RG32 to RGBA16 for output. */
uint4 PackRG32(in uint2 u2)
{
	uint x0 = (u2.x >> 16) & 0x0000ffff;
	uint y0 = (u2.y >> 16) & 0x0000ffff;
	uint x1 = (u2.x) & 0x0000ffff;
	uint y1 = (u2.y) & 0x0000ffff;
	return uint4(x1, x0, y1, y0);
}
#else
uint2 PackRG32(in uint2 u2)
{
	return u2;
}
#endif // PACK_RG32_RGBA16

#if USE_SRC_TEXTURE_ARRAY
float3 GetSourceUV(float2 UV, uint3 ThreadId) { return float3(UV, ThreadId.z); }
#else
float2 GetSourceUV(float2 UV, uint3 ThreadId) { return UV; }
#endif

#if USE_DST_TEXTURE_ARRAY
uint3 GetDestPos(uint LayerIndex, uint3 ThreadId) { return ThreadId; }
#else
uint2 GetDestPos(uint LayerIndex, uint3 ThreadId) 
{
	// clamp() shouldn't be necessary here, but has the side effect of fixing a load scheduling bug for one shader compiler (UE-304803).
	uint BatchIndex = clamp(ThreadId.z, 0, MAX_BATCH_SIZE - 1);
	uint ArrayIndex = (BatchIndex * MAX_DST_LAYERS + LayerIndex) * 2;
	uint2 BasePos = uint2(GET_SCALAR_ARRAY_ELEMENT(DestPos, ArrayIndex), GET_SCALAR_ARRAY_ELEMENT(DestPos, (ArrayIndex + 1)));
	return BasePos + ThreadId.xy; 
}
#endif

#if VERTEXSHADER
/** Trivial vertex shader for copy shaders. */
void CopyVS(in uint VertexId : SV_VertexID, out float4 OutPosition : SV_POSITION, out float2 OutTexCoord : TEXCOORD0)
{
	OutTexCoord = float2(((VertexId + 1) / 3) & 1, VertexId & 1);
	OutPosition.xy = float2(OutTexCoord.x, 1.f - OutTexCoord.y) * 2.f - 1.f;
	OutPosition.zw = float2(0, 1);
}
#endif // VERTEXSHADER

#if COMPUTESHADER
/** Compress base color to a single BC1 target. */
[numthreads(8,8,1)]
void CompressBaseColorCS(
	uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy * 4 >= SourceRect.zw))
	{
		return;
	}

	uint2 SamplePos = SourceRect.xy + ThreadId.xy * 4;
	float2 TexelUVSize = 1.f / float2(SourceRect.zw);
	float2 SampleUV = (float2(SamplePos) + 0.5f) * TexelUVSize;

	float3 BlockBaseColor[16];
	ReadBlockRGB(RenderTexture0, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockBaseColor);
#if ASTC_PROFIlE
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBlock_ETC2_SRGB(BlockBaseColor));
#elif ETC_PROFILE
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBlock_ETC2_SRGB(BlockBaseColor));
#else
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBC1Block_SRGB(BlockBaseColor));
#endif
}
#endif // COMPUTESHADER

#if PIXELSHADER
/** Simple copy of base color used for thumbnails. */
void CopyBaseColorPS(
	in float4 InPosition : SV_POSITION, 
	in noperspective float2 InTexCoord : TEXCOORD0, 
	out float4 OutColor0 : SV_Target0)
{
	float3 BaseColor = RenderTexture0.SampleLevel(TextureSampler0, InTexCoord, 0).xyz;
	OutColor0 = float4(BaseColor, 1.f);
}
#endif // PIXELSHADER

#if COMPUTESHADER
/** Compress base color, normal, roughness and specular to BC3 target pair. */
[numthreads(8,8,1)]
void CompressBaseColorNormalSpecularCS(
	uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy * 4 >= SourceRect.zw))
	{
		return;
	}

	uint2 SamplePos = SourceRect.xy + ThreadId.xy * 4;
	float2 TexelUVSize = 1.f / float2(SourceRect.zw);
	float2 SampleUV = (float2(SamplePos) + 0.5f) * TexelUVSize;

	float3 BlockBaseColor[16];
	ReadBlockRGB(RenderTexture0, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockBaseColor);

	float BlockNormalX[16];
	float BlockNormalY[16];
	ReadBlockXY(RenderTexture1, TextureSampler1, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockNormalX, BlockNormalY);

	float3 BlockSpecular[16];
	ReadBlockRGB(RenderTexture2, TextureSampler2, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockSpecular);

	for (int i=0; i<16; i++)
	{
		BlockSpecular[i].z = round(BlockSpecular[i].z);
	}
#if ASTC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ASTC_SRGBA(BlockBaseColor, BlockNormalX);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ASTC_RGBA(BlockSpecular, BlockNormalY);
#elif ETC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ETC2_SRGBA(BlockBaseColor, BlockNormalX);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ETC2_RGBA(BlockSpecular, BlockNormalY);
#else
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBC3Block_SRGB(BlockBaseColor, BlockNormalX);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBC3Block(BlockSpecular, BlockNormalY);
#endif
}
#endif // COMPUTESHADER

#if COMPUTESHADER
/** Compress base color, normal, roughness to BC1 & BC3. */
[numthreads(8, 8, 1)]
void CompressBaseColorNormalRoughnessCS(
	uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy * 4 >= SourceRect.zw))
	{
		return;
	}

	uint2 SamplePos = SourceRect.xy + ThreadId.xy * 4;
	float2 TexelUVSize = 1.f / float2(SourceRect.zw);
	float2 SampleUV = (float2(SamplePos) + 0.5f) * TexelUVSize;

	// Note that this format uses non sRGB BaseColor format with manual encode/decode.
	// So that we don't use _SRGB forms of the block compression functions below.
	float3 BlockBaseColor[16];
	ReadBlockRGB(RenderTexture0, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockBaseColor);

	float3 BlockNormalXRougnnessY[16];
	const float BlockNormalZ[16]={ 1.0, 1.0, 1.0, 1.0, 1.0, 1.0, 1.0, 1.0, 
								   1.0, 1.0, 1.0, 1.0, 1.0, 1.0, 1.0, 1.0};
	ReadBlockRGB(RenderTexture1, TextureSampler2, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockNormalXRougnnessY);
#if ASTC_PROFILE
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBlock_ETC2_RGB(BlockBaseColor));
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ASTC_RGBA(BlockNormalXRougnnessY, BlockNormalZ);
#elif ETC_PROFILE
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBlock_ETC2_RGB(BlockBaseColor));
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ETC2_RGBA(BlockNormalXRougnnessY, BlockNormalZ);
#else
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBC1Block(BlockBaseColor));
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBC3Block(BlockNormalXRougnnessY, BlockNormalZ);
#endif
}
#endif // COMPUTESHADER

#if PIXELSHADER
/** Copy path used when we disable texture compression, because we need to keep the same final channel layout. */
void CopyBaseColorNormalSpecularPS(
	in float4 InPosition : SV_POSITION, 
	in noperspective float2 InTexCoord : TEXCOORD0, 
	out float4 OutColor0 : SV_Target0,
	out float4 OutColor1 : SV_Target1)
{
	float3 BaseColor = RenderTexture0.SampleLevel(TextureSampler0, InTexCoord, 0).xyz;
	float2 NormalXY = RenderTexture1.SampleLevel(TextureSampler1, InTexCoord, 0).xy;
	float3 RoughnessSpecularNormalZ = RenderTexture2.SampleLevel(TextureSampler2, InTexCoord, 0).xyz;
	
	RoughnessSpecularNormalZ.z = round(RoughnessSpecularNormalZ.z);
	
	OutColor0 = float4(BaseColor, NormalXY.x);
	OutColor1 = float4(RoughnessSpecularNormalZ, NormalXY.y);
}
#endif // PIXELSHADER

#if COMPUTESHADER
/** Compress base color, normal, roughness and specular to BC3, BC5, BC1 target set. */
[numthreads(8,8,1)]
void CompressBaseColorNormalSpecularYCoCgCS(
	uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy * 4 >= SourceRect.zw))
	{
		return;
	}

	uint2 SamplePos = SourceRect.xy + ThreadId.xy * 4;
	float2 TexelUVSize = 1.f / float2(SourceRect.zw);
	float2 SampleUV = (float2(SamplePos) + 0.5f) * TexelUVSize;

	float3 BlockBaseColor[16];
	ReadBlockRGB(RenderTexture0, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockBaseColor);

	float BlockNormalX[16];
	float BlockNormalY[16];
	ReadBlockXY(RenderTexture1, TextureSampler1, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockNormalX, BlockNormalY);

	float3 BlockSpecular[16];
	ReadBlockRGB(RenderTexture2, TextureSampler2, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockSpecular);

	for (int i=0; i<16; i++)
	{
		BlockSpecular[i].z = round(BlockSpecular[i].z);
	}

#if ASTC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ASTC_YCoCg(BlockBaseColor);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ETC2_RG(BlockNormalX, BlockNormalY);
	OutCompressTexture2_64bit[GetDestPos(2, ThreadId)] = PackRG32(CompressBlock_ETC2_RGB(BlockSpecular));
#elif ETC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ETC2_YCoCg(BlockBaseColor);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ETC2_RG(BlockNormalX, BlockNormalY);
	OutCompressTexture2_64bit[GetDestPos(2, ThreadId)] = PackRG32(CompressBlock_ETC2_RGB(BlockSpecular));
#else
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBC3BlockYCoCg(BlockBaseColor);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBC5Block(BlockNormalX, BlockNormalY);
	OutCompressTexture2_64bit[GetDestPos(2, ThreadId)] = PackRG32(CompressBC1Block(BlockSpecular));
#endif
}
#endif // COMPUTESHADER

#if PIXELSHADER
/** Copy path used when we disable texture compression, because we need to keep the same final channel layout. */
void CopyBaseColorNormalSpecularYCoCgPS(
	in float4 InPosition : SV_POSITION, 
	in noperspective float2 InTexCoord : TEXCOORD0, 
	out float4 OutColor0 : SV_Target0,
	out float4 OutColor1 : SV_Target1,
	out float4 OutColor2 : SV_Target2)
{
	float3 YCoCg = RGB2YCoCg(RenderTexture0.SampleLevel(TextureSampler0, InTexCoord, 0).xyz);
	float2 NormalXY = RenderTexture1.SampleLevel(TextureSampler1, InTexCoord, 0).xy;
	float3 RoughnessSpecularNormalZ = RenderTexture2.SampleLevel(TextureSampler2, InTexCoord, 0).xyz;
	
	RoughnessSpecularNormalZ.z = round(RoughnessSpecularNormalZ.z);
	
	OutColor0 = float4(YCoCg.yz, 0, YCoCg.x);
	OutColor1 = float4(NormalXY, 0, 1);
	OutColor2 = float4(RoughnessSpecularNormalZ, 1);
}
#endif // PIXELSHADER

#if COMPUTESHADER
/** Compress base color, normal, roughness, specular and mask to BC3, BC5, BC3 target set. */
[numthreads(8,8,1)]
void CompressBaseColorNormalSpecularMaskYCoCgCS(
	uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy * 4 >= SourceRect.zw))
		return;

	uint2 SamplePos = SourceRect.xy + ThreadId.xy * 4;
	float2 TexelUVSize = 1.f / float2(SourceRect.zw);
	float2 SampleUV = (float2(SamplePos) + 0.5f) * TexelUVSize;

	float3 BlockBaseColor[16];
	ReadBlockRGB(RenderTexture0, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockBaseColor);

	float BlockNormalX[16];
	float BlockNormalY[16];
	float BlockMask[16];
	ReadBlockXYA(RenderTexture1, TextureSampler1, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockNormalX, BlockNormalY, BlockMask);

	float3 BlockSpecular[16];
	ReadBlockRGB(RenderTexture2, TextureSampler2, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockSpecular);

	for (int i=0; i<16; i++)
	{
		BlockSpecular[i].z = round(BlockSpecular[i].z);
	}

#if ASTC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ASTC_YCoCg(BlockBaseColor);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ETC2_RG(BlockNormalX, BlockNormalY);
	OutCompressTexture2_128bit[GetDestPos(2, ThreadId)] = CompressBlock_ASTC_RGBA(BlockSpecular, BlockMask);
#elif ETC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ETC2_YCoCg(BlockBaseColor);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBlock_ETC2_RG(BlockNormalX, BlockNormalY);
	OutCompressTexture2_128bit[GetDestPos(2, ThreadId)] = CompressBlock_ETC2_RGBA(BlockSpecular, BlockMask);
#else
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBC3BlockYCoCg(BlockBaseColor);
	OutCompressTexture1_128bit[GetDestPos(1, ThreadId)] = CompressBC5Block(BlockNormalX, BlockNormalY);
	OutCompressTexture2_128bit[GetDestPos(2, ThreadId)] = CompressBC3Block(BlockSpecular, BlockMask);
#endif
}
#endif // COMPUTESHADER

#if PIXELSHADER
/** Copy path used when we disable texture compression, because we need to keep the same final channel layout. */
void CopyBaseColorNormalSpecularMaskYCoCgPS(
	in float4 InPosition : SV_POSITION, 
	in noperspective float2 InTexCoord : TEXCOORD0, 
	out float4 OutColor0 : SV_Target0,
	out float4 OutColor1 : SV_Target1,
	out float4 OutColor2 : SV_Target2)
{
	float3 YCoCg = RGB2YCoCg(RenderTexture0.SampleLevel(TextureSampler0, InTexCoord, 0).xyz);
	float3 NormalXYMask = RenderTexture1.SampleLevel(TextureSampler1, InTexCoord, 0).xyz;
	float3 RoughnessSpecularNormalZ = RenderTexture2.SampleLevel(TextureSampler2, InTexCoord, 0).xyz;
	
	RoughnessSpecularNormalZ.z = round(RoughnessSpecularNormalZ.z);
	
	OutColor0 = float4(YCoCg.yz, 0, YCoCg.x);
	OutColor1 = float4(NormalXYMask.xy, 0, 1);
	OutColor2 = float4(RoughnessSpecularNormalZ, NormalXYMask.z);
}
#endif // PIXELSHADER

#if COMPUTESHADER
/** Compress 4 channel mask to BC3. */
[numthreads(8,8,1)]
void CompressMask4CS(
	uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy * 4 >= SourceRect.zw))
	{
		return;
	}

	uint2 SamplePos = SourceRect.xy + ThreadId.xy * 4;
	float2 TexelUVSize = 1.f / float2(SourceRect.zw);
	float2 SampleUV = (float2(SamplePos) + 0.5f) * TexelUVSize;

	float3 BlockXYZ[16];
	ReadBlockRGB(RenderTexture0, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockXYZ);

	float BlockW[16];
	ReadBlockX(RenderTexture1, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, BlockW);

#if ASTC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ASTC_RGBA(BlockXYZ, BlockW);
#elif ETC_PROFILE
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBlock_ETC2_RGBA(BlockXYZ, BlockW);
#else
	OutCompressTexture0_128bit[GetDestPos(0, ThreadId)] = CompressBC3Block(BlockXYZ, BlockW);
#endif
}
#endif // COMPUTESHADER

#if PIXELSHADER
/** Copy path used when we disable texture compression, because we need to keep the same final channel layout. */
void CopyMask4PS(
	in float4 InPosition : SV_POSITION, 
	in noperspective float2 InTexCoord : TEXCOORD0, 
	out float4 OutColor0 : SV_Target0)
{
	float3 MaskXYZ = RenderTexture0.SampleLevel(TextureSampler0, InTexCoord, 0).xyz;
	float MaskW = RenderTexture1.SampleLevel(TextureSampler1, InTexCoord, 0).x;
	
	OutColor0 = float4(MaskXYZ, MaskW);
}
#endif // PIXELSHADER

#if COMPUTESHADER
/** Compress displacement to a single BC4 target. */
[numthreads(8, 8, 1)]
void CompressDisplacementCS(
	uint3 ThreadId : SV_DispatchThreadID)
{
	if (any(ThreadId.xy * 4 >= SourceRect.zw))
	{
		return;
	}

	uint2 SamplePos = SourceRect.xy + ThreadId.xy * 4;
	float2 TexelUVSize = 1.f / float2(SourceRect.zw);
	float2 SampleUV = (float2(SamplePos) + 0.5f) * TexelUVSize;

	float Block[16];
	ReadBlockX(RenderTexture0, TextureSampler0, GetSourceUV(SampleUV, ThreadId), TexelUVSize, Block);
#if ASTC_PROFILE
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBlock_ETC2_Alpha(Block));
#elif ETC_PROFILE
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBlock_ETC2_Alpha(Block));
#else
	OutCompressTexture0_64bit[GetDestPos(0, ThreadId)] = PackRG32(CompressBC4Block(Block));
#endif
}
#endif // COMPUTESHADER

#if PIXELSHADER
/** Copy world height or displacement (without compression)
  * This path is used when want to copy the world height or displacement data into a thumbnail. */
void CopyWorldHeightPS(
	in float4 InPosition : SV_POSITION,
	in noperspective float2 InTexCoord : TEXCOORD0,
	out float4 OutColor0 : SV_Target0)
{
	float WorldHeight = RenderTexture0.SampleLevel(TextureSampler0, InTexCoord, 0).r;
	OutColor0 = float4(WorldHeight.xxx, 1.f);
}
#endif // PIXELSHADER
