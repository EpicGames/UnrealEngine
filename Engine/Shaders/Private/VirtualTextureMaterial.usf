// Copyright Epic Games, Inc. All Rights Reserved.

// Material shader for runtime virtual texture

// Remove Instanced Stereo support not needed for writing virtual texture pages.
#ifdef INSTANCED_STEREO
#undef INSTANCED_STEREO
#endif
#define INSTANCED_STEREO 0

#include "Common.ush"
#include "GammaCorrectionCommon.ush"
#include "VirtualTextureMaterial.ush"
#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

struct FVirtualTextureVSOutput
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 Position : SV_POSITION;
};

#if VERTEXSHADER

void MainVS(
	FVertexFactoryInput Input,
	out FVirtualTextureVSOutput Output
    )
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);

#if VF_PER_PIXEL_HEIGHTMAP
	if (HasVertexFactoryPerPixelHeight())
	{
		// Special case PerPixel Height for Landscape.
		// Remove PerVertex Height and apply in Pixel Shader.
		VFIntermediates.LocalPosition.z = 0;
	}
#endif

	float4 WorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);	
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, WorldPosition.xyz, TangentToLocal);

	{
		float4 RasterizedWorldPosition = VertexFactoryGetRasterizedWorldPosition(Input, VFIntermediates, WorldPosition);
		float4 ClipSpacePosition = mul(RasterizedWorldPosition, ResolvedView.TranslatedWorldToClip);
		Output.Position = ClipSpacePosition;
	}

	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);
}

#endif // VERTEXSHADER

// Prepare for VirtualTextureUnpackNormal() in VirtualTextureCommon.h
float3 PackNormal( in float3 N )
{
	return normalize(N) * (127.f / 255.f) + (127.f / 255.f);
}

// Prepare for VirtualTextureUnpackHeight() in VirtualTextureCommon.ush
// LWC_TODO: LWC scale/bias?
float PackWorldHeight( in FDFScalar Height, in float2 PackHeightScaleBias )
{
	return saturate(DFFastAddDemote(DFMultiply(Height, PackHeightScaleBias.x), PackHeightScaleBias.y));
}

void FPixelShaderInOut_MainPS(
	in FVertexFactoryInterpolantsVSToPS Interpolants,
	inout FPixelShaderIn In, 
	inout FPixelShaderOut Out )
{
	ResolvedView = ResolveView();

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants, In.SvPosition);

	FPixelMaterialInputs PixelMaterialInputs;
	CalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);

	float4 ScreenPosition = SvPositionToResolvedScreenPosition(In.SvPosition);
	float3 TranslatedWorldPosition = SvPositionToResolvedTranslatedWorld(In.SvPosition);

#if VF_PER_PIXEL_HEIGHTMAP
	if (HasVertexFactoryPerPixelHeight())
	{
		// Special case PerPixel Height for Landscape.
		// Assumes Heightmap is camera aligned.
		float3 LocalPositionDelta = float3(0, 0, GetVertexFactoryPerPixelHeight(Interpolants));
		TranslatedWorldPosition += TransformLocalVectorToWorld(MaterialParameters, LocalPositionDelta);
	}
#endif

	CalcMaterialParametersEx(MaterialParameters, PixelMaterialInputs, In.SvPosition, ScreenPosition, In.bIsFrontFace, TranslatedWorldPosition, TranslatedWorldPosition);

#if VIRTUAL_TEXTURE_OUTPUT
	// Output is from a UMaterialExpressionRuntimeVirtualTextureOutput node
	half3 BaseColor = GetVirtualTextureOutput0(MaterialParameters);
	half Specular = GetVirtualTextureOutput1(MaterialParameters);
	half Roughness = GetVirtualTextureOutput2(MaterialParameters);
	half3 Normal = GetVirtualTextureOutput3(MaterialParameters);
	FDFScalar WorldHeight = WSToDF(GetVirtualTextureOutput4_LWC(MaterialParameters));
	half Opacity = GetVirtualTextureOutput5(MaterialParameters);
	half Mask = GetVirtualTextureOutput6(MaterialParameters);
	half Displacement = GetVirtualTextureOutput7(MaterialParameters);
	half4 Mask4 = GetVirtualTextureOutput8(MaterialParameters);
#else
	// Output is from standard material output attribute node
	half3 BaseColor = GetMaterialBaseColor(PixelMaterialInputs);
	half Specular = GetMaterialSpecular(PixelMaterialInputs);
	half Roughness = GetMaterialRoughness(PixelMaterialInputs);
	half3 Normal = MaterialParameters.WorldNormal;
	FDFScalar WorldHeight = WSToDF(WSGetZ(GetWorldPosition(MaterialParameters)));
	half Opacity = GetMaterialOpacity(PixelMaterialInputs);
	half Mask = 0.f;
	half Displacement = 0.f;
	half4 Mask4 = 0.f;
#endif

	// Apply any fixed color.
	if (RuntimeVirtualTexturePassParameters.FixedColor.w > 0)
	{
		BaseColor = lerp(BaseColor, RuntimeVirtualTexturePassParameters.FixedColor.xyz, RuntimeVirtualTexturePassParameters.FixedColor.w);
		Displacement = lerp(Displacement, RuntimeVirtualTexturePassParameters.FixedColor.x, RuntimeVirtualTexturePassParameters.FixedColor.w);
	}

	// Apply mip debug color.
	if (RuntimeVirtualTexturePassParameters.MipColorAndSize.w < 1)
	{
		if (any(abs(ScreenPosition.xy) > RuntimeVirtualTexturePassParameters.MipColorAndSize.w))
		{
			BaseColor = RuntimeVirtualTexturePassParameters.MipColorAndSize.xyz;
		}
	}

#if OUT_BASECOLOR
	Out.MRT[0] = float4(BaseColor, 1.f) * Opacity;
#elif OUT_BASECOLOR_NORMAL_ROUGHNESS
	float3 PackedNormal = PackNormal(Normal);
	//store basecolor in srgb space to improve image quality
	Out.MRT[0] = float4(LinearToSrgb(BaseColor), 1.f) * Opacity;
	//not enough channels for sign of normal  z
	Out.MRT[1] = float4(PackedNormal.x, Roughness, PackedNormal.y, 1.f) * Opacity;
#elif OUT_BASECOLOR_NORMAL_SPECULAR
	float3 PackedNormal = PackNormal(Normal);
	Out.MRT[0] = float4(BaseColor, 1.f) * Opacity;
	Out.MRT[1] = float4(PackedNormal.xy, Mask, 1.f) * Opacity;
	Out.MRT[2] = float4(Specular, Roughness, PackedNormal.z, 1.f) * Opacity;
#elif OUT_MASK4
	Out.MRT[0] = float4(Mask4.xyz, 1.f) * Opacity;
	Out.MRT[1] = float4(Mask4.w, 0.f, 0.f, 1.f) * Opacity;
#elif OUT_WORLDHEIGHT
	float PackedHeight = PackWorldHeight(WorldHeight, RuntimeVirtualTexturePassParameters.PackHeight);
	Out.MRT[0] = float4(PackedHeight, 0.f, 0.f, 1.f);
#elif OUT_DISPLACEMENT
	Out.MRT[0] = float4(Displacement, 0.f, 0.f, 1.f) * Opacity;
#endif
}

#define PIXELSHADEROUTPUT_INTERPOLANTS 1

//#if defined(OUT_BASECOLOR)
//#define PIXELSHADEROUTPUT_MRT0 1
//#elif defined(OUT_BASECOLOR_NORMAL_SPECULAR)
//#define PIXELSHADEROUTPUT_MRT0 1
//#define PIXELSHADEROUTPUT_MRT1 1
//#define PIXELSHADEROUTPUT_MRT2 1
//#elif defined(OUT_MASK4)
//#define PIXELSHADEROUTPUT_MRT0 1
//#define PIXELSHADEROUTPUT_MRT1 1
//#elif defined(OUT_WORLDHEIGHT)
//#define PIXELSHADEROUTPUT_MRT0 1
//#elif defined(OUT_DISPLACEMENT)
//#define PIXELSHADEROUTPUT_MRT0 1
//#endif

// all PIXELSHADEROUTPUT_ and "void FPixelShaderInOut_MainPS()" need to be setup before this include
// this include generates the wrapper code to call MainPS(inout FPixelShaderOutput PixelShaderOutput)
#include "PixelShaderOutputCommon.ush"
