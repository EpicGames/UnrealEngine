// Copyright Epic Games, Inc. All Rights Reserved.

// Expose public values from the runtime virtual texture output pass to the material shaders.

#pragma once

#include "Common.ush"

#if !defined(IS_VIRTUAL_TEXTURE_MATERIAL)
#define IS_VIRTUAL_TEXTURE_MATERIAL 0
#endif
#if !defined(OUT_BASECOLOR)
#define OUT_BASECOLOR 0
#endif
#if !defined(OUT_BASECOLOR_NORMAL_ROUGHNESS)
#define OUT_BASECOLOR_NORMAL_ROUGHNESS 0
#endif
#if !defined(OUT_BASECOLOR_NORMAL_SPECULAR)
#define OUT_BASECOLOR_NORMAL_SPECULAR 0
#endif
#if !defined(OUT_MASK4)
#define OUT_MASK4 0
#endif
#if !defined(OUT_WORLDHEIGHT)
#define OUT_WORLDHEIGHT 0
#endif
#if !defined(OUT_DISPLACEMENT)
#define OUT_DISPLACEMENT 0
#endif

float4 GetRuntimeVirtualTextureMipLevel()
{
#if IS_VIRTUAL_TEXTURE_MATERIAL
	return RuntimeVirtualTexturePassParameters.MipLevel;
#else
	return 0;
#endif
}

float4 GetRuntimeVirtualTextureCustomData()
{
#if IS_VIRTUAL_TEXTURE_MATERIAL
	return RuntimeVirtualTexturePassParameters.CustomMaterialData;
#else
	return 0;
#endif
}

bool IsRuntimeVirtualTextureBaseColor()
{
	return (IS_VIRTUAL_TEXTURE_MATERIAL && OUT_BASECOLOR);
}

bool IsRuntimeVirtualTextureBaseColorNormalRoughness()
{
	return (IS_VIRTUAL_TEXTURE_MATERIAL && OUT_BASECOLOR_NORMAL_ROUGHNESS);
}

bool IsRuntimeVirtualTextureBaseColorNormalSpecular()
{
	return (IS_VIRTUAL_TEXTURE_MATERIAL && OUT_BASECOLOR_NORMAL_SPECULAR);
}

bool IsRuntimeVirtualTextureMask4()
{
	return (IS_VIRTUAL_TEXTURE_MATERIAL && OUT_MASK4);
}

bool IsRuntimeVirtualTextureWorldHeight()
{
	return (IS_VIRTUAL_TEXTURE_MATERIAL && OUT_WORLDHEIGHT);
}

bool IsRuntimeVirtualTextureDisplacement()
{
	return (IS_VIRTUAL_TEXTURE_MATERIAL && OUT_DISPLACEMENT);
}
