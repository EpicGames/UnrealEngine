// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Hash.ush"
#include "ColorMap.ush"

float3 IntToColor(uint Index)
{
	uint Hash = MurmurMix(Index);

	float3 Color = float3
	(
		(Hash >>  0) & 255,
		(Hash >>  8) & 255,
		(Hash >> 16) & 255
	);

	return Color * (1.0f / 255.0f);
}

float3 HUEtoRGB(in float H)
{
	float R = abs(H * 6 - 3) - 1;
	float G = 2 - abs(H * 6 - 2);
	float B = 2 - abs(H * 6 - 4);
	return saturate(float3(R, G, B));
}

float3 HSVtoRGB(in float3 HSV)
{
	float3 RGB = HUEtoRGB(HSV.x);
	return ((RGB - 1) * HSV.y + 1) * HSV.z;
}

float3 GetColorCode(float x)
{
	float c = (1 - saturate(x)) * 0.6; // Remap [0,1] to Blue-Red
	return x > 0 ? HSVtoRGB(float3(c, 1, 1)) : float3(0, 0, 0);
}

float3 GreenToRedHUE(float s)
{
	return HUEtoRGB(lerp(0.333333f, 0.0f, saturate(s)));
}

float3 GreenToRedTurbo(float s)
{
	return ColorMapTurbo(lerp(0.5f, 1.0f, saturate(s)));
}

float3 BlueGreenRedColorMap(float t)
{
	return (0.75f * t + 0.25f) * saturate(2.0 - abs(float3(4, 2, 0) - 4.0 * t));
}
