// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================================
	VisualizeRayBuffer.usf: Prepare occlusion rays for a given directional light
===============================================================================================*/

#include "Common.ush"
#include "DeferredShadingCommon.ush"
#include "RayTracing/RayTracingCommon.ush"

StructuredBuffer<FBasicRayData> RayBuffer;
RWTexture2D<float3> RWRayBufferUAV;

uint BufferOffsetAtPixel(uint2 Pixel)
{
	return Pixel.y * View.BufferSizeAndInvSize.x + Pixel.x;
}

[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void MainCS(
	uint3 DispatchThreadId : SV_DispatchThreadID
)
{
	uint2 LaunchIndex = DispatchThreadId.xy;
	uint BufferOffset = BufferOffsetAtPixel(LaunchIndex);

	float3 Origin = RayBuffer[BufferOffset].Origin;
	float3 Direction = RayBuffer[BufferOffset].Direction;
	float TFar = RayBuffer[BufferOffset].TFar;
	uint Mask = RayBuffer[BufferOffset].Mask;

	RWRayBufferUAV[LaunchIndex] = Origin;
	//RWRayBufferUAV[LaunchIndex] = float3(1, 1, 0);
}