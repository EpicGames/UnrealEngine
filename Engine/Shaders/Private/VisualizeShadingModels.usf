// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeferredShadingCommon.ush"
#include "PostProcessCommon.ush"

Texture2D SceneColorTexture;
SamplerState SceneColorSampler;

DECLARE_SCALAR_ARRAY(uint, ShadingModelMaskInView, 16);

void MainPS(noperspective float4 UVAndScreenPos : TEXCOORD0, float4 SvPosition : SV_POSITION, out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	int2 PixelPos = (int2)SvPosition.xy;

	float3 SceneColor = Texture2DSample(SceneColorTexture, SceneColorSampler, UV).rgb;

	FGBufferData GBufferData = GetGBufferData(UV);

	float3 ShadingModelColor = GetShadingModelColor(GBufferData.ShadingModelID);

	OutColor.xyz = ShadingModelColor * lerp(0.8f, 1.0f, Luminance(SceneColor));

	{
		const int2 LegendLeftTop = int2(32, 160);
		const int YStep = 20;
		const int ShadingModelCount = SHADINGMODELID_NUM;
		const int2 LegendSize = int2(220, ShadingModelCount * YStep);

		// (0, 0) .. (1, 1)
		float2 InsetPx = PixelPos - LegendLeftTop;

		float BorderDistance = ComputeDistanceToRect(PixelPos, LegendLeftTop, LegendSize);

		float3 LegendColor = 0.25f;	// dark grey

		const int ShadingModelID = (InsetPx.y * ShadingModelCount) / LegendSize.y;

		if (ShadingModelID < ShadingModelCount)
		{
			const bool bValue = GET_SCALAR_ARRAY_ELEMENT(ShadingModelMaskInView, ShadingModelID) != 0;

			float3 LegendShadingModelColor = GetShadingModelColor(ShadingModelID);

			if(InsetPx.x < 20)
			{
				LegendColor = LegendShadingModelColor;
			}
			else
			{
				LegendColor = bValue ? 0.5f : 0.2f;
			}
		}

		// thin black border around the legend
		OutColor.xyz = lerp(float3(0, 0, 0), OutColor.xyz, saturate(BorderDistance - 20));
		OutColor.xyz = lerp(LegendColor, OutColor.xyz, saturate(BorderDistance - 1));
		OutColor.w = 1.0f;
	}
}