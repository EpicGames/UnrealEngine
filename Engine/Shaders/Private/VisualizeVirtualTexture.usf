// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "PostProcessCommon.ush"
#include "ScreenPass.ush"
 
SCREEN_PASS_TEXTURE_VIEWPORT(Input)
SCREEN_PASS_TEXTURE_VIEWPORT(Output)

Texture2D InputTexture;
SamplerState InputSampler;

StructuredBuffer<uint> DebugBuffer;
uint2 ViewSize;

uint ViewMode;

float4 Colors[MAX_NUM_COLORS];
uint ColorCount;

void Main(
	noperspective float4 UVAndScreenPos : TEXCOORD0,
	FStereoVSOutput StereoOutput,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0)
{
	float2 UV = UVAndScreenPos.xy;
	float3 SceneColor = Texture2DSample(InputTexture, InputSampler, UV).rgb;
	float SceneColorLuminance = saturate(Luminance(SceneColor));
	OutColor.xyz = SceneColorLuminance;
	OutColor.w = 1;

	uint2 PixelPos = (uint2)(saturate(UV) * (float2)(ViewSize - 1));
	uint ReadIndex = PixelPos.y * ViewSize.x + PixelPos.x;
	uint PackedDebugValue = DebugBuffer[ReadIndex];
	uint DebugValue = 0;

	if (ViewMode == 1)
	{
		// Pending mips in lower 16 bits.
		DebugValue = PackedDebugValue & 0xffff;
	}
	else if (ViewMode == 2)
	{
		// Stack count in upper 16 bits.
		DebugValue = PackedDebugValue >> 16;
	}

	if (DebugValue != 0)
	{
		DebugValue = clamp(DebugValue - 1, 0, ColorCount - 1);
		OutColor.xyz = Colors[DebugValue].rgb * lerp(0.8, 1.0, SceneColorLuminance);
	}
}
