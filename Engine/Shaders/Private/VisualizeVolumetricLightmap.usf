// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"
#include "SHCommon.ush"
#include "VolumetricLightmapShared.ush"
#include "DeferredShadingCommon.ush"

#ifndef QUADS_PER_INSTANCE
#define QUADS_PER_INSTANCE 0
#endif

struct FVisualizeVolumetricLightmapVSToPS
{
	nointerpolation float3 BrickUVs : TEXCOORD0;
	nointerpolation float FadeToAmbient : TEXCOORD1;
	nointerpolation float4 CellSphere : TEXCOORD2;
	float3 VertexPosition : TEXCOORD3;
}; 

float VisualizationRadiusScale;
float VisualizationMinScreenFraction;

void VisualizeVolumetricLightmapVS(
	uint VertexId : SV_VertexID,
	uint InstanceId : SV_InstanceID, 
	out FVisualizeVolumetricLightmapVSToPS Output,
	out float4 OutPosition : SV_POSITION
	)
{
	uint EffectiveInstanceId = InstanceId * QUADS_PER_INSTANCE + VertexId / 4;

	uint3 NumHighestDensityCells = View.VolumetricLightmapIndirectionTextureSize * View.VolumetricLightmapBrickSize;

	uint3 CellVector = uint3(
		EffectiveInstanceId % NumHighestDensityCells.x,
		(EffectiveInstanceId / NumHighestDensityCells.x) % NumHighestDensityCells.y,
		EffectiveInstanceId / (NumHighestDensityCells.x * NumHighestDensityCells.y));

	float3 IndirectionTextureTexelCoordinate = CellVector / View.VolumetricLightmapBrickSize;
	float4 BrickOffsetAndSize = View.VolumetricLightmapIndirectionTexture.Load(int4(IndirectionTextureTexelCoordinate, 0));

	float PaddedBrickSize = View.VolumetricLightmapBrickSize + 1;
	float3 LocalCellPosition = frac(IndirectionTextureTexelCoordinate / BrickOffsetAndSize.w) * View.VolumetricLightmapBrickSize;
	Output.BrickUVs = (BrickOffsetAndSize.xyz * PaddedBrickSize + LocalCellPosition + .5f) * View.VolumetricLightmapBrickTexelSize;

	float3 GridLocalCellPosition = round(LocalCellPosition);
	bool bValidCell = all(abs(LocalCellPosition - GridLocalCellPosition) < .01f);

	Output.CellSphere.xyz = (CellVector / (float3)NumHighestDensityCells - View.VolumetricLightmapWorldToUVAdd) / View.VolumetricLightmapWorldToUVScale;
	float DetailVoxelWorldSize = 1.0f / ((float)View.VolumetricLightmapIndirectionTextureSize.x * View.VolumetricLightmapWorldToUVScale.x);
	// Set visualization sphere radius as a fraction of the corresponding brick's world space size
	Output.CellSphere.w = BrickOffsetAndSize.w * DetailVoxelWorldSize * VisualizationRadiusScale;
	
	{
		float DistanceToCamera = length(Output.CellSphere.xyz - DFHackToFloat(PrimaryView.WorldViewOrigin));
		float SizeOnScreen = Output.CellSphere.w * View.ViewToClip[0][0] / DistanceToCamera;
		Output.FadeToAmbient = saturate(SizeOnScreen * 100 - .01f);
		// Maintain a minimum size on screen to reduce aliasing
		SizeOnScreen = max(SizeOnScreen, VisualizationMinScreenFraction * sqrt(BrickOffsetAndSize.w));
		Output.CellSphere.w = SizeOnScreen * DistanceToCamera / View.ViewToClip[0][0];
	}
	
	float2 TexCoord = float2(
		((VertexId >> 0) & 1) ? 1.0f : -1.0f,
		((VertexId >> 1) & 1) ? 1.0f : -1.0f);

	float2 RescaledTexCoord = TexCoord * Output.CellSphere.w * sqrt(2.0);

	Output.VertexPosition = Output.CellSphere.xyz + RescaledTexCoord.x * View.ViewRight + RescaledTexCoord.y * View.ViewUp;
	OutPosition = mul(float4(Output.VertexPosition, 1), DFHackToFloat(PrimaryView.WorldToClip));
	
	if (!bValidCell)
	{
		// This cell is not resident, emit a degenerate triangle
		OutPosition = 0;
	}
}

#define WRITE_TO_SHADING_MODEL (FEATURE_LEVEL >= FEATURE_LEVEL_SM4 && !FORWARD_SHADING)

float3 DiffuseColor;

void VisualizeVolumetricLightmapPS(
	FVisualizeVolumetricLightmapVSToPS Input,
	out float4 OutColor : SV_Target0
#if WRITE_TO_SHADING_MODEL
	,out float4 OutGBufferB : SV_Target1
#endif
)
{
	float3 RayDirection = Input.VertexPosition - DFHackToFloat(PrimaryView.WorldViewOrigin);
	float2 SphereIntersections = RayIntersectSphere(DFHackToFloat(PrimaryView.WorldViewOrigin), RayDirection, Input.CellSphere);
	float3 IntersectionPosition = DFHackToFloat(PrimaryView.WorldViewOrigin) + SphereIntersections.x * RayDirection;
	clip(SphereIntersections.x);

	FThreeBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH3(Input.BrickUVs);

	float3 WorldNormal = normalize(IntersectionPosition - Input.CellSphere.xyz);
	FThreeBandSHVector DiffuseTransferSH = CalcDiffuseTransferSH3(WorldNormal, 1);
	float3 SphereLighting = max(float3(0,0,0), DotSH3(IrradianceSH, DiffuseTransferSH)) / PI;

	float3 AmbientLighting = float3(IrradianceSH.R.V0.x, IrradianceSH.G.V0.x, IrradianceSH.B.V0.x) / PI;
	// Lerp to ambient lighting in the distance to reduce aliasing
	float3 FinalLighting = lerp(AmbientLighting, SphereLighting, Input.FadeToAmbient);

	FinalLighting *= View.PreExposure;

	// Use a diffuse color that matches the rest of the scene for LightingOnly and DetailLighting view modes
	float3 EffectiveDiffuseColor = DiffuseColor * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
	OutColor = float4(EffectiveDiffuseColor * FinalLighting, 0);

	//OutColor = float4(WorldNormal, 0);

#if WRITE_TO_SHADING_MODEL
	// Shading model unlit
	SetGBufferForUnlit(OutGBufferB);

#endif
}