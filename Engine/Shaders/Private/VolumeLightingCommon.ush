// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * VolumeLightingCommon.usf
 */

// use low quality shadow sampling on translucency for better performance
#define SHADOW_QUALITY 2

#include "ShadowProjectionCommon.ush"
#include "ShadowFilteringCommon.ush"

/** Parameters needed to access the shadow map of the light. */
float4x4 TranslatedWorldToShadowMatrix;
float4 ShadowmapMinMax;
// .x:1/SplitNearFadeRegion, .y:1/SplitFarFadeRegion .zw:DistanceFadeMAD
float4 ShadowInjectParams;
// .x:DepthBias, .y:SlopeDepthBias, .z:MaxSlopeDepthBias, .w:Projection parameters
float4 DepthBiasParameters;

/** Whether to compute static shadowing. */
uint bStaticallyShadowed;

/** Shadow depth map computed for static geometry by Lightmass. */
Texture2D StaticShadowDepthTexture;
SamplerState StaticShadowDepthTextureSampler;

/** Transform used for static shadowing by spot and directional lights. */
float4x4 TranslatedWorldToStaticShadowMatrix;
float4 StaticShadowBufferSize;

// Declare shadow sampling function
#include "VolumeLightingCommonSampling.ush"


