// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * VolumeLightingCommonSampling.usf
 */

/** Computes dynamic and static shadowing for a point anywhere in space. */
#ifdef VOLUME_SHADOW_SAMPLING_INPUT

#if VOLUME_SHADOW_SAMPLING_INPUT==0

// Sample from Light0Shadow
#define VSSPREFIX(X) Light0Shadow.X
#define VSSLIGHTPOSITION  Light0Shadow.TranslatedWorldPosition
#define VSSLIGHTINVRADIUS Light0Shadow.InvRadius

float ComputeLight0VolumeShadowing(

#elif VOLUME_SHADOW_SAMPLING_INPUT==1

// Sample from VolumeLightingCommon constants
#define VSSPREFIX(X) Light1Shadow.X
#define VSSLIGHTPOSITION  Light1Shadow.TranslatedWorldPosition
#define VSSLIGHTINVRADIUS Light1Shadow.InvRadius

float ComputeLight1VolumeShadowing(

#else
#error Unhandled value for VOLUME_SHADOW_SAMPLING_INPUT
#endif

#else

// Sample from data in 
#define VSSPREFIX(X)  X
#define VSSLIGHTPOSITION  DeferredLightUniforms.TranslatedWorldPosition
#define VSSLIGHTINVRADIUS DeferredLightUniforms.InvRadius

float ComputeVolumeShadowing(

#endif
	float3 TranslatedWorldPositionForLighting, bool bPointLight, bool bSpotLight, inout bool bShadowFactorValid)
{
	float ShadowFactor = 1;
	bShadowFactorValid = false;

	BRANCH
	if (VSSPREFIX(bStaticallyShadowed))
	{
		bool bUsePointLightShadowing = bPointLight;

		BRANCH
		if (bUsePointLightShadowing)
		{
			float3 LightVector = TranslatedWorldPositionForLighting - VSSLIGHTPOSITION;
			float DistanceToLight = length(LightVector);
			float3 NormalizedLightVector = LightVector / DistanceToLight;

			//@todo - use parametrization without slow inverse trig.  Dual paraboloid?
			float NormalizedTheta = atan2(NormalizedLightVector.y, NormalizedLightVector.x) / (2 * PI);
			// atan2 returns in the range [-PI, PI], wrap the negative portion to [.5, 1]
			float U = NormalizedTheta > 0 ? NormalizedTheta : 1 + NormalizedTheta;
			float V = acos(NormalizedLightVector.z) / PI;
			float2 UnwrappedUVs = float2(U, V);

			float ShadowDepth = Texture2DSampleLevel(VSSPREFIX(StaticShadowDepthTexture), VSSPREFIX(StaticShadowDepthTextureSampler), UnwrappedUVs, 0).x;
			ShadowFactor = DistanceToLight * VSSLIGHTINVRADIUS < ShadowDepth;
			bShadowFactorValid = true;
		}
		else
		{
			// This path is used for directional lights and spot lights, which only require a single projection
			// Transform the world position into shadowmap space
			float4 HomogeneousShadowPosition = mul(float4(TranslatedWorldPositionForLighting, 1), VSSPREFIX(TranslatedWorldToStaticShadowMatrix));
			float2 ShadowUVs = HomogeneousShadowPosition.xy / HomogeneousShadowPosition.w;

			// Treat as unshadowed if the voxel is outside of the shadow map
			if (all(and(ShadowUVs >= 0, ShadowUVs <= 1)))
			{
				FPCFSamplerSettings Settings;
				Settings.ShadowDepthTexture = VSSPREFIX(StaticShadowDepthTexture);
				Settings.ShadowDepthTextureSampler = VSSPREFIX(StaticShadowDepthTextureSampler);
				Settings.ShadowBufferSize = VSSPREFIX(StaticShadowBufferSize);
				Settings.SceneDepth = HomogeneousShadowPosition.z;
				Settings.TransitionScale = 40;
				Settings.bSubsurface = false;
				// We can sample outside of the static shadowmap, which is centered around the scene.  These 'infinite' depth values should not cause occlusion.
				Settings.bTreatMaxDepthUnshadowed = true;
				Settings.DensityMulConstant = 0;
				Settings.ProjectionDepthBiasParameters = float2(0, 0);

				ShadowFactor = Manual1x1PCF(ShadowUVs, Settings);
				bShadowFactorValid = true;

				/*
				// Sample the shadowmap depth and determine if this voxel is shadowed
				float ShadowDepth = Texture2DSampleLevel(StaticShadowDepthTexture, StaticShadowDepthTextureSampler, ShadowUVs, 0).x;
				ShadowFactor = HomogeneousShadowPosition.z < ShadowDepth;
				*/
			}
		}
	}		

#if DYNAMICALLY_SHADOWED && FEATURE_LEVEL >= FEATURE_LEVEL_SM4
	bool bUseCubemapShadowing = bPointLight;
	float DynamicShadowFactor = 1;

	if (bUseCubemapShadowing)
	{
		bShadowFactorValid = true;
		const float DepthBias = 0.03f * 512 * VSSPREFIX(InvShadowmapResolution);
		const float SlopeDepthBias = 0;
		const float MaxSlopeDepthBias = 0;

		DynamicShadowFactor = CubemapHardwarePCF(
			VSSPREFIX(ShadowDepthCubeTexture), VSSPREFIX(ShadowDepthCubeTextureSampler), VSSPREFIX(ShadowViewProjectionMatrices), VSSPREFIX(InvShadowmapResolution),
			TranslatedWorldPositionForLighting, VSSLIGHTPOSITION, VSSLIGHTINVRADIUS, DepthBias, SlopeDepthBias, MaxSlopeDepthBias);
	}
	else
	{
		// Transform the world position into shadowmap space
		// NOTE: Shadow space is reverse Z, but shadowmap is stored in "linear Z"
		float4 HomogeneousShadowPosition = mul(float4(TranslatedWorldPositionForLighting, 1), VSSPREFIX(TranslatedWorldToShadowMatrix));
		float2 ShadowUVs = HomogeneousShadowPosition.xy / HomogeneousShadowPosition.w;
		float SceneDepth = 1.0f - HomogeneousShadowPosition.z;

		// Treat as unshadowed if the voxel is outside of the shadow map
		if (all(ShadowUVs >= VSSPREFIX(ShadowmapMinMax).xy) && all(ShadowUVs <= VSSPREFIX(ShadowmapMinMax).zw))
		{
			bShadowFactorValid = true;
			// Sample the shadowmap depth and determine if this voxel is shadowed
			float ShadowDepth = Texture2DSampleLevel(VSSPREFIX(ShadowDepthTexture), VSSPREFIX(ShadowDepthTextureSampler), ShadowUVs, 0).x;
			DynamicShadowFactor = SceneDepth < ShadowDepth - VSSPREFIX(DepthBiasParameters).x;

#ifdef TREAT_MAXDEPTH_UNSHADOWED
			DynamicShadowFactor = saturate(DynamicShadowFactor + (ShadowDepth == 1.0f));
#endif
		}
	}

	// fade shadows in the distance
	if (!bPointLight && !bSpotLight)
	{
		float Depth = dot(TranslatedWorldPositionForLighting - PrimaryView.TranslatedWorldCameraOrigin, View.ViewForward);
		float DistanceFade = saturate(Depth * VSSPREFIX(ShadowInjectParams).z + VSSPREFIX(ShadowInjectParams).w);
		DynamicShadowFactor = lerp(DynamicShadowFactor, 1.0f, DistanceFade * DistanceFade);
	}

	// Combine static shadowing and dynamic shadowing, important for stationary directional lights with CSM
	ShadowFactor = min(ShadowFactor, DynamicShadowFactor);

#endif

	return ShadowFactor;
}



#undef VSSPREFIX
#undef VSSLIGHTPOSITION
#undef VSSLIGHTINVRADIUS

