// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * VolumetricCloud.usf: Render volumetric cloud on screen.    
 */  


 // Change this to force recompilation of all volumetric cloud material shaders
#pragma message("UESHADERMETADATA_VERSION 26C4467C-50E5-414D-86B5-3F3C27A85891")

// Use reservoir sampling for any virtual texture feedback because the number of samples in the tracing loop is not known upfront.
#define VT_FEEDBACK_RESERVOIR_SAMPLE 1

#include "Common.ush"
#include "ColorMap.ush"
#include "ColorSpace.ush"

#define SceneTexturesStruct RenderVolumetricCloudParameters.SceneTextures

#include "Random.ush"
#include "VolumetricCloudCommon.ush"
#include "/Engine/Shared/EnvironmentComponentsFlags.h"

#ifndef SHADER_RENDERVIEW_CS
#define SHADER_RENDERVIEW_CS 0
#endif

#ifndef SHADER_RENDERVIEW_PS
#define SHADER_RENDERVIEW_PS 0
#endif

#ifndef SHADER_EMPTY_SPACE_SKIPPING_CS
#define SHADER_EMPTY_SPACE_SKIPPING_CS 0
#endif

#ifndef VIRTUAL_SHADOW_MAP
#define VIRTUAL_SHADOW_MAP 0
#endif

#ifndef CLOUD_SAMPLE_LOCAL_LIGHTS
#define CLOUD_SAMPLE_LOCAL_LIGHTS 0
#endif

#ifndef EMPTY_SPACE_SKIPPING_DEBUG
#define EMPTY_SPACE_SKIPPING_DEBUG 0
#endif

#ifndef MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT
#define MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT 0
#endif

#ifndef CLOUD_DEBUG_SAMPLES
#define CLOUD_DEBUG_SAMPLES 0
#endif

#ifndef CLOUD_DEBUG_VIEW_MODE
#define CLOUD_DEBUG_VIEW_MODE 0
#endif

#ifndef CLOUD_MIN_AND_MAX_DEPTH
#define CLOUD_MIN_AND_MAX_DEPTH 0
#endif

#ifndef PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
#define PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME 0
#endif

#ifndef SHADOW_DEPTH_SHADER
#define SHADOW_DEPTH_SHADER 0
#endif

// Draw debug sample positions
#if CLOUD_DEBUG_SAMPLES || EMPTY_SPACE_SKIPPING_DEBUG
#include "ShaderPrint.ush"
#endif

#if defined(SHADER_MAINVS) || defined(SHADER_SHADOW_PS) || SHADER_RENDERVIEW_PS || SHADER_RENDERVIEW_CS || SHADER_EMPTY_SPACE_SKIPPING_CS

#include "/Engine/Generated/Material.ush"
#include "VolumetricCloudMaterialPixelCommon.ush"

#if !SHADER_RENDERVIEW_CS && !SHADER_EMPTY_SPACE_SKIPPING_CS
#include "/Engine/Generated/VertexFactory.ush"
#endif

#include "ReflectionEnvironmentShared.ush"
#include "ParticipatingMediaCommon.ush"

#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
#define LFVStruct RenderVolumetricCloudParameters
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif

#if defined(SHADER_SHADOW_PS) || SHADER_RENDERVIEW_PS || SHADER_RENDERVIEW_CS || SHADER_EMPTY_SPACE_SKIPPING_CS

void VolumetricCloudCalcPixelMaterialInputs(in out FMaterialPixelParameters Parameters, in out FPixelMaterialInputs PixelMaterialInputs)
{
	// This same logic is also used in CalcMaterialParameters(), which is also called in these shaders, so we keep it here for consistency.
#if USE_ANALYTIC_DERIVATIVES
	if (!TEXTURE_SAMPLE_DEBUG || View.GeneralPurposeTweak >= 1.0f)
		CalcPixelMaterialInputsAnalyticDerivatives(Parameters, PixelMaterialInputs);
	else
#endif
	{
		CalcPixelMaterialInputs(Parameters, PixelMaterialInputs);
	}
}

float3 SampleExtinctionCoefficients(in FPixelMaterialInputs PixelMaterialInputs)
{
	float3 Extinction = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	Extinction = VOLUMETRICFOGCLOUD_EXTINCTION(BSDF).rgb;
#else
	#if !MATERIAL_SHADINGMODEL_UNLIT
	Extinction = GetMaterialSubsurfaceDataRaw(PixelMaterialInputs).rgb;
	#endif
#endif
	return clamp(Extinction, 0.0f, 65000.0f);
}

float3 SampleEmissive(in FPixelMaterialInputs PixelMaterialInputs)
{
	float3 EmissiveColor = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	EmissiveColor = BSDF_GETEMISSIVE(BSDF).rgb;
#else
	EmissiveColor = GetMaterialEmissiveRaw(PixelMaterialInputs).rgb;
#endif
	return clamp(EmissiveColor, 0.0f, 65000.0f);
}

float3 SampleAlbedo(in FPixelMaterialInputs PixelMaterialInputs)
{
	float3 Albedo = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	Albedo = VOLUMETRICFOGCLOUD_ALBEDO(BSDF).rgb;
#else
	#if !MATERIAL_SHADINGMODEL_UNLIT
	Albedo = GetMaterialBaseColor(PixelMaterialInputs).rgb * View.DiffuseOverrideParameter.w + View.DiffuseOverrideParameter.xyz;
	#endif
#endif
	return saturate(Albedo);
}

float SampleAmbientOcclusion(in FPixelMaterialInputs PixelMaterialInputs)
{
	float AO = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	AO = VOLUMETRICFOGCLOUD_AO(BSDF);
#else
	AO = GetMaterialAmbientOcclusion(PixelMaterialInputs);
#endif
	return AO;
}

void ConvertCloudPixelMaterialInputsToWorkingColorSpace(inout FPixelMaterialInputs PixelMaterialInputs)
{
#if !WORKING_COLOR_SPACE_IS_SRGB
	// Get raw data
	float3 ExtinctionSRGB = SampleExtinctionCoefficients(PixelMaterialInputs);
	float3 AlbedoSRGB = SampleAlbedo(PixelMaterialInputs);
	float3 EmissiveSRGB = SampleEmissive(PixelMaterialInputs);

	/** To be revisited:
	 * Converting albedo and extinction directly appears to yield better results than calculating scattering
	 * and absorption, converting those to the working color space and extracting the results back.
	 * The (ScatteringSRGB = AlbedoSRGB * ExtinctionSRGB) multiplication is sensitive to its color space context.
	 */
	float3 ExtinctionWCS	= SRGBCoefficientsToWorkingColorSpace(ExtinctionSRGB);
	float3 AlbedoWCS		= SRGBColorToWorkingColorSpace(AlbedoSRGB);
	float3 EmissiveWCS		= SRGBColorToWorkingColorSpace(EmissiveSRGB);

#if TEMPLATE_USES_SUBSTRATE	
	VOLUMETRICFOGCLOUD_EXTINCTION(PixelMaterialInputs.FrontMaterial.InlinedBSDF)	= ExtinctionWCS;
	VOLUMETRICFOGCLOUD_ALBEDO(PixelMaterialInputs.FrontMaterial.InlinedBSDF)		= AlbedoWCS;
	BSDF_GETEMISSIVE(PixelMaterialInputs.FrontMaterial.InlinedBSDF)					= EmissiveWCS;
#else
	// Set raw values back
	PixelMaterialInputs.Subsurface.rgb	= ExtinctionWCS;
	PixelMaterialInputs.BaseColor		= AlbedoWCS;
	PixelMaterialInputs.EmissiveColor	= EmissiveWCS;
#endif // SUBSTRATE_ENABLED
#endif // WORKING_COLOR_SPACE_IS_SRGB
}

#endif

#endif

#if CLOUD_SAMPLE_LOCAL_LIGHTS
#define ForwardLightStruct RenderVolumetricCloudParameters.Forward
#include "LightGridCommon.ush"
#define SUPPORT_CONTACT_SHADOWS 0
#include "DeferredLightingCommon.ush"
#include "LightData.ush"
#endif



////////////////////////////////////////////////////////////
// Single full screen triangle vertex shader
////////////////////////////////////////////////////////////

#ifdef SHADER_MAINVS

void MainVS(
	float3 InPosition : ATTRIBUTE0,
	out float4 Position : SV_POSITION
)
{
	ResolvedView = ResolveView();

	Position = float4(InPosition, 1.0f);
}

#endif // SHADER_MAINVS



////////////////////////////////////////////////////////////
// Common structures and functions
////////////////////////////////////////////////////////////

bool RayIntersectSphereSolution(float3 RayOrigin, float3 RayDirection, float4 Sphere, inout float2 Solutions)
{
	float3 LocalPosition = RayOrigin - Sphere.xyz;
	float LocalPositionSqr = dot(LocalPosition, LocalPosition);

	float3 QuadraticCoef;
	QuadraticCoef.x = dot(RayDirection, RayDirection);
	QuadraticCoef.y = 2 * dot(RayDirection, LocalPosition);
	QuadraticCoef.z = LocalPositionSqr - Sphere.w * Sphere.w;

	float Discriminant = QuadraticCoef.y * QuadraticCoef.y - 4 * QuadraticCoef.x * QuadraticCoef.z;

	// Only continue if the ray intersects the sphere
	FLATTEN
	if (Discriminant >= 0)
	{
		float SqrtDiscriminant = sqrt(Discriminant);
		Solutions = (-QuadraticCoef.y + float2(-1, 1) * SqrtDiscriminant) / (2 * QuadraticCoef.x);
		return true;
	}

	return false;
}



////////////////////////////////////////////////////////////
// Single full screen triangle vertex shader
////////////////////////////////////////////////////////////

#if SHADER_RENDERVIEW_PS || SHADER_RENDERVIEW_CS || SHADER_EMPTY_SPACE_SKIPPING_CS

#include "/Engine/Private/SkyAtmosphereCommon.ush"

#define BlueNoise RenderVolumetricCloudParameters.BlueNoise
#include "BlueNoise.ush"

#define FogStruct RenderVolumetricCloudParameters
#undef MATERIALBLENDING_ADDITIVE	// we need to override this for height fog to appear
#define MATERIALBLENDING_ADDITIVE 0
#include "HeightFogCommon.ush"
#undef MATERIALBLENDING_ADDITIVE
#define MATERIALBLENDING_ADDITIVE 1	// Restor that back because volumetric material must be additive

#ifndef MATERIAL_VOLUMETRIC_ADVANCED
#define MATERIAL_VOLUMETRIC_ADVANCED 0
#endif
#ifndef MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERPIXEL
#define MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERPIXEL 0
#endif
#ifndef MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERSAMPLE
#define MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERSAMPLE 0
#endif
#ifndef MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT
#define MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT 0
#endif
#ifndef MATERIAL_VOLUMETRIC_ADVANCED_GRAYSCALE_MATERIAL
#define MATERIAL_VOLUMETRIC_ADVANCED_GRAYSCALE_MATERIAL 0
#endif
#ifndef MATERIAL_VOLUMETRIC_ADVANCED_RAYMARCH_VOLUME_SHADOW
#define MATERIAL_VOLUMETRIC_ADVANCED_RAYMARCH_VOLUME_SHADOW 0
#endif
#ifndef MATERIAL_VOLUMETRIC_ADVANCED_CLAMP_MULTISCATTERING_CONTRIBUTION
#define MATERIAL_VOLUMETRIC_ADVANCED_CLAMP_MULTISCATTERING_CONTRIBUTION 1
#endif
#ifndef MATERIAL_VOLUMETRIC_ADVANCED_OVERRIDE_AMBIENT_OCCLUSION
#define MATERIAL_VOLUMETRIC_ADVANCED_OVERRIDE_AMBIENT_OCCLUSION 0
#endif
#ifndef MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT
#define MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT 0
#endif


#if CLOUD_SAMPLE_ATMOSPHERIC_LIGHT_SHADOWMAP

#define DYNAMICALLY_SHADOWED 1
#define TREAT_MAXDEPTH_UNSHADOWED 1

#define SHADOW_QUALITY 2
#define NO_TRANSLUCENCY_AVAILABLE

#include "ShadowProjectionCommon.ush"
#include "ShadowFilteringCommon.ush"

#define VOLUME_SHADOW_SAMPLING_INPUT 0
// We cannot have Light0Shadow as an additional global parameter structure when rendering a MeshMaterialShader (only one PassUniformBuffer).
// So instead it has been included and redirected using a macro here. This however prevents us to have two atmospheric lights casting shadow at the same time...
#define Light0Shadow RenderVolumetricCloudParameters
#include "VolumeLightingCommonSampling.ush"
#undef VOLUME_SHADOW_SAMPLING_INPUT
#undef Light0Shadow

#if VIRTUAL_SHADOW_MAP
#include "VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#endif

#endif

float SamplePhaseFunction(in float PhaseCosTheta, in float PhaseG, in float PhaseG2, in float PhaseBlend)
{
	PhaseG = clamp(PhaseG, -0.999f, 0.999f);
	PhaseG2 = clamp(PhaseG2, -0.999f, 0.999f);
	PhaseBlend = clamp(PhaseBlend, 0.0f, 1.0f);
	float MiePhaseValueLight0 = HenyeyGreensteinPhase(PhaseG, -PhaseCosTheta);	// negate cosTheta because due to WorldDir being a "in" direction. 
	float MiePhaseValueLight1 = HenyeyGreensteinPhase(PhaseG2, -PhaseCosTheta);
	const float Phase = MiePhaseValueLight0 + PhaseBlend * (MiePhaseValueLight1 - MiePhaseValueLight0);
	return Phase;
}


// Multi scattering approximation based on http://magnuswrenninge.com/wp-content/uploads/2010/03/Wrenninge-OzTheGreatAndVolumetric.pdf
// 1 is for the default single scattering look. Then [2,N] is for extra "octaves"
#ifndef MSCOUNT
#define MSCOUNT (1 + MATERIAL_VOLUMETRIC_ADVANCED_MULTISCATTERING_OCTAVE_COUNT)
#endif

#if MATERIAL_VOLUMETRIC_ADVANCED_GRAYSCALE_MATERIAL

#define MATVEC              float
#define ColorToMATVEC(C)    ((C).r)   // Assume all 3 floats are the same
#define MATVEC_Max(C)       (C)
#define MATVEC_Min(C)       (C)
#define MATVEC_Avg(C)       (C)

#else

#define MATVEC              float3
#define ColorToMATVEC(C)    ((C).rgb)
float MATVEC_Max(float3 C) { return max3(C.r, C.g, C.b); }
float MATVEC_Min(float3 C) { return min3(C.r, C.g, C.b); }
float MATVEC_Avg(float3 C) { return dot(C, 1.0f / 3.0f); }

#endif

struct ParticipatingMediaContext
{
	MATVEC ScatteringCoefficients[MSCOUNT];
	MATVEC ExtinctionCoefficients[MSCOUNT];

	float3 TransmittanceToLight0[MSCOUNT];
#if CLOUD_SAMPLE_SECOND_LIGHT
	float3 TransmittanceToLight1[MSCOUNT];
#endif
};

ParticipatingMediaContext SetupParticipatingMediaContext(MATVEC BaseAlbedo, MATVEC BaseExtinctionCoefficients, float MsSFactor, float MsEFactor, float3 InitialTransmittanceToLight0, float3 InitialTransmittanceToLight1)
{
	const MATVEC ScatteringCoefficients = BaseAlbedo * BaseExtinctionCoefficients;
	//const float3 AbsorptionCoefficients = max(0.0f, BaseExtinctionCoefficients - ScatteringCoefficients);

	ParticipatingMediaContext PMC;
	PMC.ScatteringCoefficients[0] = ScatteringCoefficients;
	PMC.ExtinctionCoefficients[0] = BaseExtinctionCoefficients;
	PMC.TransmittanceToLight0[0] = InitialTransmittanceToLight0;
#if CLOUD_SAMPLE_SECOND_LIGHT
	PMC.TransmittanceToLight1[0] = InitialTransmittanceToLight1;
#endif

	UNROLL
	for (int ms = 1; ms < MSCOUNT; ++ms)
	{
		PMC.ScatteringCoefficients[ms] = PMC.ScatteringCoefficients[ms - 1] * MsSFactor;
		PMC.ExtinctionCoefficients[ms] = PMC.ExtinctionCoefficients[ms - 1] * MsEFactor;
		MsSFactor *= MsSFactor;
		MsEFactor *= MsEFactor;

		PMC.TransmittanceToLight0[ms] = InitialTransmittanceToLight0;
#if CLOUD_SAMPLE_SECOND_LIGHT
		PMC.TransmittanceToLight1[ms] = InitialTransmittanceToLight1;
#endif
	}

	return PMC;
}


struct ParticipatingMediaPhaseContext
{
	float Phase0[MSCOUNT];
#if CLOUD_SAMPLE_SECOND_LIGHT
	float Phase1[MSCOUNT];
#endif
};

ParticipatingMediaPhaseContext SetupParticipatingMediaPhaseContext(float BasePhase0, float BasePhase1, float MsPhaseFactor)
{
	ParticipatingMediaPhaseContext PMPC;
	PMPC.Phase0[0] = BasePhase0;
#if CLOUD_SAMPLE_SECOND_LIGHT
	PMPC.Phase1[0] = BasePhase1;
#endif

	UNROLL
	for (int ms = 1; ms < MSCOUNT; ++ms)
	{
		PMPC.Phase0[ms] = lerp(IsotropicPhase(), PMPC.Phase0[0], MsPhaseFactor);
#if CLOUD_SAMPLE_SECOND_LIGHT
		PMPC.Phase1[ms] = lerp(IsotropicPhase(), PMPC.Phase1[0], MsPhaseFactor);
#endif
		MsPhaseFactor *= MsPhaseFactor;
	}

	return PMPC;
}

#define SHADER_RENDERVIEW_CS_WITH_EMPTY_SPACE_SKIPPING (SHADER_RENDERVIEW_CS && MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT)

#if SHADER_RENDERVIEW_CS_WITH_EMPTY_SPACE_SKIPPING

float2	StartTracingDistanceTextureResolution;
float StartTracingSampleVolumeDepth;
Texture2D<float> StartTracingDistanceTexture;

#endif // SHADER_RENDERVIEW_CS_WITH_EMPTY_SPACE_SKIPPING

#define GETAERIALPERSPECTIVE(t, ts, x, o, f) GetAerialPerspectiveLuminanceTransmittance(\
		ResolvedView.RealTimeReflectionCapture, ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeSizeAndInvSize,\
		ClipPos, (x - ResolvedView.TranslatedWorldCameraOrigin) * CM_TO_SKY_UNIT,\
		t, ts,\
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolutionInv,\
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthResolution,\
		o,\
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKm,\
		ResolvedView.SkyAtmosphereCameraAerialPerspectiveVolumeDepthSliceLengthKmInv,\
		ResolvedView.OneOverPreExposure,\
		f);\

void MainCommon(in FMaterialPixelParameters MaterialParameters, in float4 SvPosition, 
	inout float4 OutColor0
#if CLOUD_MIN_AND_MAX_DEPTH
	, inout float4 OutColor1 
#endif
	, inout float4 OutDepth
	, inout float  OutHoldOutCoverage
	, inout float  OutAboveCloudHoldOut)
{
	ResolvedView = ResolveView();

//#if 0
//	const float displaySize = 256.0f;
//	if(all(SvPosition.xy<displaySize))
//	{
//		OutColor0 = float4(RenderVolumetricCloudParameters.CloudShadowTexture.Load(uint3(SvPosition.xy,0), 0).rg, 0.0, 0.0f);
//		return;
//	}
//#endif


	//
	// Initialise all the parameters
	//
	OutColor0					= float4(0.0f, 0.0f, 0.0f, 1.0f);
	OutDepth					= MaxHalfFloat;

	// CalcMaterialParameters() evaluates the material, but since the depth associated with SvPosition at this point doesn't correspond to any particular point along the ray,
	// we leave WorldPosition_DDX/Y as 0, which forces mip 0 when compiled with USE_ANALYTIC_DERIVATIVES.
	FPixelMaterialInputs PixelMaterialInputs	= (FPixelMaterialInputs)0;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, true);

	float3 RayTranslatedWorldOrigin		= GetTranslatedWorldCameraPosFromView(SvPosition.xy);
	float3 RayTranslatedWorldOriginKm	= RayTranslatedWorldOrigin * CENTIMETER_TO_KILOMETER;
	float3 RayWorldOriginKm				= select(IsOrthoProjection(), DFHackToFloat(DFSubtract(RayTranslatedWorldOrigin, PrimaryView.PreViewTranslation)) * CENTIMETER_TO_KILOMETER, DFHackToFloat(ResolvedView.WorldCameraOrigin) * CENTIMETER_TO_KILOMETER);
	float3 Raydir						= -MaterialParameters.CameraVector;

	float TMin							= -999999999.0f;
	float TMax							= -999999999.0f;

	float3 Luminance					= 0.0f;
	MATVEC TransmittanceToView			= 1.0f;
	float tAPWeightedSum				= 0.0f;
	float tAPWeightsSum					= 0.0f;

#if CLOUD_MIN_AND_MAX_DEPTH
	OutColor1 = float4(0.0f, 0.0f, 0.0f, 1.0f);
	float3 ClosestLuminance = 0.0f;
	MATVEC ClosestTransmittanceToView = 1.0f;
	float ClostestTAPWeightedSum = 0.0f;
	float ClostestTAPWeightsSum = 0.0f;
#endif // CLOUD_MIN_AND_MAX_DEPTH

#if USE_ANALYTIC_DERIVATIVES
	const float3 TranslatedWorldPositionOffsetX = SvPositionToResolvedTranslatedWorld(SvPosition + float4(1.0f, 0.0f, 0.0f, 0.0f));
	const float3 TranslatedWorldPositionOffsetY = SvPositionToResolvedTranslatedWorld(SvPosition + float4(0.0f, 1.0f, 0.0f, 0.0f));
	const float3 RayDirOffsetX = GetCameraVector(ResolvedView, TranslatedWorldPositionOffsetX, ResolvedView.TranslatedWorldCameraOrigin);
	const float3 RayDirOffsetY = GetCameraVector(ResolvedView, TranslatedWorldPositionOffsetY, ResolvedView.TranslatedWorldCameraOrigin);

	const float3 RayTranslatedWorldOriginOffsetX = GetTranslatedWorldCameraPosFromView(ResolvedView, SvPosition.xy + float2(1.0f, 0.0f));
	const float3 RayTranslatedWorldOriginOffsetY = GetTranslatedWorldCameraPosFromView(ResolvedView, SvPosition.xy + float2(0.0f, 1.0f));
#endif // USE_ANALYTIC_DERIVATIVES


	////////////////////////////////////////////////////////////////////////////////////////////////////
	// Begin the cloud ray marching and setup the variable above
	////////////////////////////////////////////////////////////////////////////////////////////////////
	{
		//
		// Check tracing start and end position within the cloud layer
		//
		float2 tTop2 = -1.0f;
		float2 tBottom2 = -1.0f;
		if (RayIntersectSphereSolution(RayWorldOriginKm, Raydir, float4(RenderVolumetricCloudParameters.CloudLayerCenterKm, RenderVolumetricCloudParameters.TopRadiusKm), tTop2))
		{
			if (RayIntersectSphereSolution(RayWorldOriginKm, Raydir, float4(RenderVolumetricCloudParameters.CloudLayerCenterKm, RenderVolumetricCloudParameters.BottomRadiusKm), tBottom2))
			{
				// If we see both intersection in front of us, keep the min/closest, otherwise the max/furthest
				float TempTop = all(tTop2 > 0.0f) ? min(tTop2.x, tTop2.y) : max(tTop2.x, tTop2.y);
				float TempBottom = all(tBottom2 > 0.0f) ? min(tBottom2.x, tBottom2.y) : max(tBottom2.x, tBottom2.y);

				if (all(tBottom2 > 0.0f))
				{
					// But if we can see the bottom of the layer, make sure we use the camera or the highest top layer intersection
					TempTop = max(0.0f, min(tTop2.x, tTop2.y));
				}

				TMin = min(TempBottom, TempTop);
				TMax = max(TempBottom, TempTop);
			}
			else
			{
				// Only intersecting with the top atmosphere, we have our min and max t
				TMin = tTop2.x;
				TMax = tTop2.y;
			}
		}
		else
		{
			// No intersection with at least the top of the atmosphere
			//OutColor0 = float4(1.0f, 0.0f, 0.0f, 1.0f);
			return;
		}
		TMin = max(0.0f, TMin) * KILOMETER_TO_CENTIMETER;
		TMax = max(0.0f, TMax) * KILOMETER_TO_CENTIMETER;

		const bool bDoDepthIntersectionAndTest = RenderVolumetricCloudParameters.OpaqueIntersectionMode >= 2;
		if (IsOrthoProjection())
		{
			//Ortho is always drawing to the far plane for now, so clip anything that is in front of it when we are inside the sky sphere. Earlies out compared to the further checks below.
			if (bDoDepthIntersectionAndTest && length(RayWorldOriginKm - RenderVolumetricCloudParameters.CloudLayerCenterKm) < RenderVolumetricCloudParameters.TopRadiusKm && OutDepth.z > 0.0f)
			{
				return;
			}
		}

		//
		// Skip tracing if the range is 0, e.g. cloud layer is behind an object and we do not want to pay for SampleCountMin, or the distance at which tracing should start it too far away
		//

		if (TMax <= TMin || TMin > RenderVolumetricCloudParameters.TracingStartMaxDistance)
		{
			//OutColor0 = float4(1.0f, 0.0f, 1.0f, 1.0f);
			return;
		}



		//
		// Sample the depth buffer and update tracing distance
		//

		uint4 TracingCoordToZbufferCoordScaleBias = RenderVolumetricCloudParameters.TracingCoordToZbufferCoordScaleBias;
		uint2 SceneDepthTextureCoord = uint2(SvPosition.xy - 0.5) * TracingCoordToZbufferCoordScaleBias.xy + TracingCoordToZbufferCoordScaleBias.zw;
		SceneDepthTextureCoord = clamp(SceneDepthTextureCoord, RenderVolumetricCloudParameters.SceneDepthTextureMinMaxCoord.xy, RenderVolumetricCloudParameters.SceneDepthTextureMinMaxCoord.zw);

#if CLOUD_MIN_AND_MAX_DEPTH

		float2 MinAndMaxDepth = RenderVolumetricCloudParameters.SceneDepthMinAndMaxTexture.Load(uint3(SceneDepthTextureCoord / 2, 0)).rg;
		MinAndMaxDepth = max(0.000000000001, MinAndMaxDepth.yx); // x is clostest depth, y is furthest depth

		// Trace up to furthest depth.
		float DeviceZ = MinAndMaxDepth.y;

		float3 ClostestDepthBufferTranslatedWorldPos = SvPositionToTranslatedWorld(float4(SvPosition.xy, MinAndMaxDepth.x, 1.0));
		const float TClosestDepthBufferKm = min(length(ClostestDepthBufferTranslatedWorldPos * CENTIMETER_TO_KILOMETER - RayTranslatedWorldOriginKm), MaxHalfFloat);
		const float TClosestDepthBufferCm = TClosestDepthBufferKm * KILOMETER_TO_CENTIMETER;
		float3 FurthestDepthBufferTranslatedWorldPos = SvPositionToTranslatedWorld(float4(SvPosition.xy, MinAndMaxDepth.y, 1.0));
		const float TFurthestDepthBufferKm = min(length(FurthestDepthBufferTranslatedWorldPos * CENTIMETER_TO_KILOMETER - RayTranslatedWorldOriginKm), MaxHalfFloat);

		OutDepth.xyzw = float4(TFurthestDepthBufferKm, TFurthestDepthBufferKm, TClosestDepthBufferKm, TFurthestDepthBufferKm);

		const float TDepthBufferCm = TFurthestDepthBufferKm * KILOMETER_TO_CENTIMETER;

#else // CLOUD_MIN_AND_MAX_DEPTH

		float DeviceZ = RenderVolumetricCloudParameters.SceneDepthTexture.Load(uint3(SceneDepthTextureCoord, 0)).r;
		DeviceZ = max(0.000000000001, DeviceZ);
		float3 DepthBufferTranslatedWorldPos = SvPositionToTranslatedWorld(float4(SvPosition.xy, DeviceZ, 1.0));
		const float TDepthBufferKm = min(length(DepthBufferTranslatedWorldPos * CENTIMETER_TO_KILOMETER - RayTranslatedWorldOriginKm), MaxHalfFloat);
		OutDepth.xyzw = float4(TDepthBufferKm, TDepthBufferKm, TDepthBufferKm, TDepthBufferKm);

		const float TDepthBufferCm = TDepthBufferKm * KILOMETER_TO_CENTIMETER;

#endif // CLOUD_MIN_AND_MAX_DEPTH

		//
		// Skip tracing if the depth buffer is in front of the cloud layer front interface. If inside the layer, clamp the tracing to the depth buffer
		//

		if(!IsOrthoProjection() || TMax < View.OrthoFarPlane)
		{
			if (bDoDepthIntersectionAndTest && TDepthBufferCm < TMin)
			{
				//OutColor0 = float4(0.0f, 0.0f, 1.0f, 1.0f);
				return;
			}

			if (bDoDepthIntersectionAndTest)
			{
				// Only trace up to the closest between the cloud layer far distance and the depth buffer 
				TMax = min(TMax, TDepthBufferCm);
			}
		}

		//
		// Prepare a bunch of variable for the tracing
		//

#if CLOUD_PER_SAMPLE_ATMOSPHERE_TRANSMITTANCE 
		const float3 Light0Illuminance = ResolvedView.AtmosphereLightIlluminanceOuterSpace[0].rgb;
#else
		const float3 Light0Illuminance = ResolvedView.AtmosphereLightIlluminanceOnGroundPostTransmittance[0].rgb;
#endif
		const float3 Light0IlluminanceFinal = Light0Illuminance * (RenderVolumetricCloudParameters.EnableAtmosphericLightsSampling ? RenderVolumetricCloudParameters.AtmosphericLightCloudScatteredLuminanceScale[0].rgb : float3(0.0f, 0.0f, 0.0f));
		const float3 Light0Direction = ResolvedView.AtmosphereLightDirection[0].xyz;

#if CLOUD_SAMPLE_SECOND_LIGHT
#if CLOUD_PER_SAMPLE_ATMOSPHERE_TRANSMITTANCE 
		const float3 Light1Illuminance = ResolvedView.AtmosphereLightIlluminanceOuterSpace[1].rgb;
#else
		const float3 Light1Illuminance = ResolvedView.AtmosphereLightIlluminanceOnGroundPostTransmittance[1].rgb;
#endif
		const float3 Light1IlluminanceFinal = Light1Illuminance * (RenderVolumetricCloudParameters.EnableAtmosphericLightsSampling ? RenderVolumetricCloudParameters.AtmosphericLightCloudScatteredLuminanceScale[1].rgb : float3(0.0f, 0.0f, 0.0f));
		const float3 Light1Direction = ResolvedView.AtmosphereLightDirection[1].xyz;
#else // CLOUD_SAMPLE_SECOND_LIGHT
		const float3 Light1Illuminance = float3(0.0f, 0.0f, 0.0f);
		const float3 Light1IlluminanceFinal = float3(0.0f, 0.0f, 0.0f);
		const float3 Light1Direction = float3(0.0f, 0.0f, 0.0f);
#endif // CLOUD_SAMPLE_SECOND_LIGHT


		const bool bTracingMaxDistanceModeFromCamera = RenderVolumetricCloudParameters.TracingMaxDistanceMode == 1;

		// Clamp to tracing max distance according to the selected mode
		if (RenderVolumetricCloudParameters.TracingMaxDistanceMode == 0)
		{
			// Distance after the cloud layer entry point
			const float MarchingDistance = min(RenderVolumetricCloudParameters.TracingMaxDistance, TMax - TMin);
			TMax = TMin + MarchingDistance;
		}
		else // if (RenderVolumetricCloudParameters.TracingMaxDistanceMode == 1)
		{
			// Distance from the point of view
			TMin = min(RenderVolumetricCloudParameters.TracingMaxDistance, TMin);

			const float MaxBottom = max(tBottom2.x, tBottom2.y) * KILOMETER_TO_CENTIMETER;
			if (all(tBottom2 > 0.0f) && (MaxBottom <= RenderVolumetricCloudParameters.TracingMaxDistance))
			{
				// Make sure we trace when traveling out of the bottom entry point but travel back again in for the specified range of distance.
				// But we try to not trace outside of the cloud layer still for highly curved planets.
				TMax = min(RenderVolumetricCloudParameters.TracingMaxDistance, max(tTop2.x, tTop2.y) * KILOMETER_TO_CENTIMETER);

				// And we must apply the far, i.e. not near, depth buffer constraint to TMax.
				TMax = min(TMax, TDepthBufferCm);
			}
			else
			{
				TMax = min(TMax, RenderVolumetricCloudParameters.TracingMaxDistance);
			}

			// Exit if tracing a 0 length segment.
			if ((TMax - TMin) <= 0.0f)
			{
				//OutColor0 = float4(1.0f, 1.0f, 0.0f, 1.0f);
				return;
			}
		}

		// Account for the tracing start distance
		TMin = max(TMin, RenderVolumetricCloudParameters.TracingStartDistanceFromCamera);
		TMax = max(TMin, TMax);

		// When we do trace, we want a minimum amount of sample count. This avoid situation with 0 sample count due to InvDistanceToSampleCountMax.
		const uint  EvaluatedIStepCount = max(RenderVolumetricCloudParameters.SampleCountMin, RenderVolumetricCloudParameters.SampleCountMax * saturate((TMax - TMin) * RenderVolumetricCloudParameters.InvDistanceToSampleCountMax));
		const float StepCount = float(EvaluatedIStepCount);
		const float StepT = (TMax - TMin) / StepCount; // StepT is step distance in centimeters
		const float dtMeters = StepT * CENTIMETER_TO_METER;

		// After the sample count for a trace has been evaluated, we can still clamp the sample count.
		// This is useful to avoid long running GPU waves which can increase the cost, especially when started later on the async compute pipe.
		// It should not be set too low otherwise the visual result will look incomplete.
		const uint  IStepCount = min(EvaluatedIStepCount, RenderVolumetricCloudParameters.SampleCountClamp);

		// This is the scattered sky light luminance assuming a uniform phase function.
		// TODO Approximate some form of occlusion
		// TODO have some directionality: use SH from sky atmosphere? Or GetSkySHDiffuse/GetSkySHDiffuseSimple for dynamic SkyLightCapture.
		float3 DistantSkyLightLuminance = 0.0f;
		if (RenderVolumetricCloudParameters.EnableDistantSkyLightSampling)
		{
			const bool bSkyAtmospherePresentInScene = ResolvedView.SkyAtmospherePresentInScene > 0.0f;
			if (bSkyAtmospherePresentInScene)
			{
				// TODO Only works for ground view. This should be spatially varying for space viewsspace view. 
				// TODO Have a illuminance texture to get that overall ambient contribution? 
				// TODO An approximation would be to multiply it with sun transmittance?
				DistantSkyLightLuminance = GetViewDistanceSkyLightColor();
			}
			else
			{
				// That unfortunately will create a bad feedback loop since clouds will be captured in the sky light.
				DistantSkyLightLuminance = GetSkySHDiffuseSimple(float3(0.0f, 0.0f, 0.0f)); // without SH directionality
			}
		}



		// Evalaute Factors once per pixel
#if MATERIAL_VOLUMETRIC_ADVANCED
	#if MATERIAL_VOLUMETRIC_ADVANCED_CLAMP_MULTISCATTERING_CONTRIBUTION
		const float MsScattFactor = saturate(GetVolumetricAdvancedMaterialOutput3(MaterialParameters));
	#else
		const float MsScattFactor = GetVolumetricAdvancedMaterialOutput3(MaterialParameters);
	#endif
		const float MsExtinFactor = saturate(GetVolumetricAdvancedMaterialOutput4(MaterialParameters));
		const float MsPhaseFactor = saturate(GetVolumetricAdvancedMaterialOutput5(MaterialParameters));
#else
		const float MsScattFactor = 1.0f;
		const float MsExtinFactor = 1.0f;
		const float MsPhaseFactor = 1.0f;
#endif

#if CLOUD_DEBUG_VIEW_MODE
		uint DebugZeroConservativeDensitySampleCount = 0;
#endif

		const float3 wi0 = Light0Direction;
		const float3 wi1 = Light1Direction;
		const float3 wo = Raydir;
		const float Phase0CosTheta = dot(wi0, wo);
		const float Phase1CosTheta = dot(wi1, wo);
#if MATERIAL_VOLUMETRIC_ADVANCED
#if MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERPIXEL
		const float PhaseG = GetVolumetricAdvancedMaterialOutput0(MaterialParameters);
		const float PhaseG2 = GetVolumetricAdvancedMaterialOutput1(MaterialParameters);
		const float PhaseBlend = GetVolumetricAdvancedMaterialOutput2(MaterialParameters);
		const float Phase0 = SamplePhaseFunction(Phase0CosTheta, PhaseG, PhaseG2, PhaseBlend);
#if CLOUD_SAMPLE_SECOND_LIGHT
		const float Phase1 = SamplePhaseFunction(Phase1CosTheta, PhaseG, PhaseG2, PhaseBlend);
#else // CLOUD_SAMPLE_SECOND_LIGHT
		const float Phase1 = IsotropicPhase();
#endif // CLOUD_SAMPLE_SECOND_LIGHT
		ParticipatingMediaPhaseContext PMPC = SetupParticipatingMediaPhaseContext(Phase0, Phase1, MsPhaseFactor);
#endif
#else
		ParticipatingMediaPhaseContext PMPC = SetupParticipatingMediaPhaseContext(IsotropicPhase(), IsotropicPhase(), MsPhaseFactor);
#endif

		FCloudLayerParameters CloudLayerParams = GetCloudLayerParams(
			RenderVolumetricCloudParameters.CloudLayerCenterKm, RenderVolumetricCloudParameters.PlanetRadiusKm,
			RenderVolumetricCloudParameters.BottomRadiusKm, RenderVolumetricCloudParameters.TopRadiusKm);

		//
		// TRACING LOOP
		//

	#if CLOUD_DEBUG_SAMPLES
		int2 DebugPixelCoord = float2(ResolvedView.CursorPosition - RenderVolumetricCloudParameters.TracingCoordToFullResPixelCoordScaleBias.zw) / float2(RenderVolumetricCloudParameters.TracingCoordToFullResPixelCoordScaleBias.xy);
		const bool bDrawDebugEnabled = all(int2(SvPosition.xy) == DebugPixelCoord);
	#endif

		float t = TMin + 0.5 * StepT;
		if (RenderVolumetricCloudParameters.IsReflectionRendering == 0)
		{
		#if 0 // Simple noise
			t = TMin + (float(Rand3DPCG16(int3(SvPosition.xy, View.StateFrameIndexMod8)).x) * rcp(65536.0)) * StepT;
		#else // Blue noise works best
			uint2 FullResPixelCoord = SvPosition.xy * RenderVolumetricCloudParameters.TracingCoordToFullResPixelCoordScaleBias.xy; // bias is not needed, noise will still repeat as needed
			t = TMin + BlueNoiseScalar(FullResPixelCoord, View.StateFrameIndexMod8) * StepT;
		#endif
		}

	#if SHADER_RENDERVIEW_CS_WITH_EMPTY_SPACE_SKIPPING
		// In this case we cannot run variable step size to make sure all our samples are aligned and empty space skipping low resolution texture texel do nto become visible.
		const float StepSizeOnZeroConservativeDensity = 1.0f;
	#else
		const float StepSizeOnZeroConservativeDensity = RenderVolumetricCloudParameters.StepSizeOnZeroConservativeDensity;
	#endif

		//
		// Read from empty space skipping
		//
		uint StartStepCount = 0;
	#if SHADER_RENDERVIEW_CS_WITH_EMPTY_SPACE_SKIPPING
		const float StartDepthKm = StartTracingDistanceTexture[SvPositionToViewportUV(SvPosition) * StartTracingDistanceTextureResolution];
		const float StartDepthCm = StartDepthKm * KILOMETER_TO_CENTIMETER;
	#if !CLOUD_DEBUG_VIEW_MODE
		if (t < StartDepthCm)
		{
			StartStepCount = uint(StartDepthCm / StepT);
			t += float(StartStepCount) * StepSizeOnZeroConservativeDensity * StepT;
		}
		//OutColor0 = float4(StartDepthKm / 50.0f, 0.0f, 0.0f, 0.5f);
		//return;
	#endif
	#endif

		float3 S0 = 0.0f;
		for (uint i = StartStepCount; i < IStepCount; ++i)
		{
			int ms;
			float3 SampleTranslatedWorldPosition = RayTranslatedWorldOrigin + t * Raydir;
#if CLOUD_MIN_AND_MAX_DEPTH
			const bool SampleContributeToClosest = t <= TClosestDepthBufferCm;
#endif // CLOUD_MIN_AND_MAX
			 
#if USE_ANALYTIC_DERIVATIVES
			float3 SampleWorldPositionDDX = (RayTranslatedWorldOriginOffsetX + t * RayDirOffsetX) - SampleTranslatedWorldPosition;
			float3 SampleWorldPositionDDY = (RayTranslatedWorldOriginOffsetY + t * RayDirOffsetY) - SampleTranslatedWorldPosition;
#else
			float3 SampleWorldPositionDDX = 0.0f;
			float3 SampleWorldPositionDDY = 0.0f;
#endif // USE_ANALYTIC_DERIVATIVES

			// Using the same derivatives for all shadow samples is an approximation only and could possibly be improved in the future
			const float3 ShadowSampleWorldPositionDDX = SampleWorldPositionDDX;
			const float3 ShadowSampleWorldPositionDDY = SampleWorldPositionDDY;

			//////////////////////////////
			// Update FMaterialPixelParameters according to current sample.
			//////////////////////////////
			UpdateMaterialCloudParam(MaterialParameters, SampleTranslatedWorldPosition, SampleWorldPositionDDX, SampleWorldPositionDDY,
				ResolvedView, CloudLayerParams, TRACING_SHADOW_DISTANCE_OFF, RenderVolumetricCloudParameters.EmptySpaceSkippingSliceDepth);

#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY && !CLOUD_DEBUG_VIEW_MODE
			if (MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
			{
				i += StepSizeOnZeroConservativeDensity - 1;
				t += StepSizeOnZeroConservativeDensity * StepT;

			#if CLOUD_DEBUG_SAMPLES
				if (bDrawDebugEnabled) AddCrossTWS(SampleTranslatedWorldPosition, 60000.0f, float4(1, 0, 0, 0.5));
			#endif
				continue; // Conservative density is 0 so skip and go to the next sample
			}
			#if CLOUD_DEBUG_SAMPLES
			else
			{
				if(bDrawDebugEnabled) AddCrossTWS(SampleTranslatedWorldPosition, 60000.0f, float4(0, 1, 0, 0.5));
			}
			#endif // CLOUD_DEBUG_SAMPLES
#endif

#if CLOUD_DEBUG_VIEW_MODE
			if (RenderVolumetricCloudParameters.CloudDebugViewMode == 1 && MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
			{
				++DebugZeroConservativeDensitySampleCount;
			}
		#if SHADER_RENDERVIEW_CS_WITH_EMPTY_SPACE_SKIPPING
			else if (RenderVolumetricCloudParameters.CloudDebugViewMode == 2 && GetVolumetricCloudEmptySpaceSkippingOutput0(MaterialParameters).x <= 0.0f)
			{
				++DebugZeroConservativeDensitySampleCount;
			}
		#endif
#endif

			if (MaterialParameters.CloudSampleNormAltitudeInLayer <= 0.0f || MaterialParameters.CloudSampleNormAltitudeInLayer >= 1.0f)
			{
				// If we are out of the cloud volume, for instance when bTracingMaxDistanceModeFromCamera is true, 
				// we can and must ignore all material to not change the look in case the material graph is not fading out the clouds below the layer.
				t += StepT;
				continue;
			}

			//////////////////////////////
			// Sample the participating media material at the sample position.
			//////////////////////////////
			VolumetricCloudCalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
			ConvertCloudPixelMaterialInputsToWorkingColorSpace(PixelMaterialInputs);
			const MATVEC ExtinctionCoefficients = ColorToMATVEC(SampleExtinctionCoefficients(PixelMaterialInputs));
			const MATVEC EmissiveLuminance = USES_EMISSIVE_COLOR ? ColorToMATVEC(SampleEmissive(PixelMaterialInputs)) : 0.0f;
			const MATVEC Albedo = ColorToMATVEC(SampleAlbedo(PixelMaterialInputs));

#if MATERIAL_VOLUMETRIC_ADVANCED_PHASE_PERSAMPLE
			const float PhaseG = GetVolumetricAdvancedMaterialOutput0(MaterialParameters);
			const float PhaseG2 = GetVolumetricAdvancedMaterialOutput1(MaterialParameters);
			const float PhaseBlend = GetVolumetricAdvancedMaterialOutput2(MaterialParameters);
			const float Phase0 = SamplePhaseFunction(Phase0CosTheta, PhaseG, PhaseG2, PhaseBlend);
#if CLOUD_SAMPLE_SECOND_LIGHT
			const float Phase1 = SamplePhaseFunction(Phase1CosTheta, PhaseG, PhaseG2, PhaseBlend);
#else // CLOUD_SAMPLE_SECOND_LIGHT
			const float Phase1 = IsotropicPhase();
#endif // CLOUD_SAMPLE_SECOND_LIGHT
			ParticipatingMediaPhaseContext PMPC = SetupParticipatingMediaPhaseContext(Phase0, Phase1, MsPhaseFactor);
#endif

			const float3 PlanetCenterToWorldPos = (SampleTranslatedWorldPosition - ResolvedView.SkyPlanetTranslatedWorldCenterAndViewHeight.xyz) * CM_TO_SKY_UNIT;
#if CLOUD_PER_SAMPLE_ATMOSPHERE_TRANSMITTANCE
			// Apply per sample change of transmittance due to atmosphere. More expenssive but higher quality, (and required for space view)
			const float3 AtmosphereTransmittanceToLight0 = GetAtmosphereTransmittance(
				PlanetCenterToWorldPos, Light0Direction, ResolvedView.SkyAtmosphereBottomRadiusKm, ResolvedView.SkyAtmosphereTopRadiusKm,
				View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler);
#else
			const float3 AtmosphereTransmittanceToLight0 = 1.0f;
#endif

#if CLOUD_SAMPLE_SECOND_LIGHT && CLOUD_PER_SAMPLE_ATMOSPHERE_TRANSMITTANCE
			// Apply per sample change of transmittance due to atmosphere. More expenssive but higher quality, (and required for space view)
			const float3 AtmosphereTransmittanceToLight1 = GetAtmosphereTransmittance(
				PlanetCenterToWorldPos, Light1Direction, ResolvedView.SkyAtmosphereBottomRadiusKm, ResolvedView.SkyAtmosphereTopRadiusKm,
				View.TransmittanceLutTexture, View.TransmittanceLutTextureSampler);
#else
			const float3 AtmosphereTransmittanceToLight1 = 1.0f;
#endif // CLOUD_SAMPLE_SECOND_LIGHT


			ParticipatingMediaContext PMC = SetupParticipatingMediaContext(Albedo, ExtinctionCoefficients, MsScattFactor, MsExtinFactor, AtmosphereTransmittanceToLight0, AtmosphereTransmittanceToLight1);


			// We always apply the sky distant luminance as computed by the SkyAtmosphere component for a given altitude. 
			// TODO: this should be spatially varying accoring to height and sun angle.
			float3 DistantLightLuminance = DistantSkyLightLuminance;
#if MATERIAL_VOLUMETRIC_ADVANCED_OVERRIDE_AMBIENT_OCCLUSION
			// We reduce the sky contibution as specified by the user instead of the default behavior.
			DistantLightLuminance *= SampleAmbientOcclusion(PixelMaterialInputs);
#else
			// We reduce the sky contibution at the bottom of the cloud using a cheap gradient that is super fast to apply and artist controlable.
			DistantLightLuminance *= saturate(RenderVolumetricCloudParameters.SkyLightCloudBottomVisibility + MaterialParameters.CloudSampleNormAltitudeInLayer);
#endif


			//////////////////////////////
			// Evaluate some data if there is medium causing scattering, e.g. shadow, ground lighting
			//////////////////////////////
			if (any(PMC.ScatteringCoefficients[0] > 0.0f))
			{
				const float MaxTransmittanceToView = MATVEC_Max(TransmittanceToView);

				//
				// A- we compute lighting bouncing off the ground: affected by light direction, transmittance in the atmosphere and albedo.
				//
#if MATERIAL_VOLUMETRIC_ADVANCED_GROUND_CONTRIBUTION
				if (MaxTransmittanceToView > 0.01f)
				{
					// Cheap approximation assuming a single transmittance using only current sample extinction value. If we are withint the AO texture bounds, we lerp towards its more accurate result.
					//DistantLightLuminance += TransmittedScatteredLightLuminance;

					MATVEC OpticalDepth = 0.0f;
					const float ShadowLengthTest = min(5.0 * KILOMETER_TO_CENTIMETER, MaterialParameters.CloudSampleAltitudeInLayer);
					const float ShadowStepCount = 5.0f;
					const float InvShadowStepCount = 1.0f / ShadowStepCount;

					// Evaluate the direction toward the ground only once
					const float3 GroundNormal    = normalize(PlanetCenterToWorldPos);	// Ambient contribution from the clouds is only done on a plane above the planet, e.g. do not support space view yet
					const float3 GroundDirection = -GroundNormal;

#if 1
					// Non-linear shadow sample distribution, hardcoded to x^2
					float PreviousNormT = 0.0f;
					for (float ShadowT = InvShadowStepCount; ShadowT <= 1.00001f; ShadowT += InvShadowStepCount)
					{
						float CurrentNormT = ShadowT * ShadowT; // CurrentNormT is the end of the considered segment to integrate, PreviousNormT is its beginning.
						const float DetlaNormT = CurrentNormT - PreviousNormT;
						const float ShadowSampleDistance = ShadowLengthTest * (CurrentNormT - 0.5 * DetlaNormT);
						UpdateMaterialCloudParam(MaterialParameters, SampleTranslatedWorldPosition + GroundDirection * ShadowSampleDistance, ShadowSampleWorldPositionDDX, ShadowSampleWorldPositionDDY,
							ResolvedView, CloudLayerParams, ShadowSampleDistance, SPACE_SKIPPING_SLICE_DEPTH_OFF);
						PreviousNormT = CurrentNormT;
#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY
						if (MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
						{
							continue; // Conservative density is 0 so skip and go to the next sample
						}
#endif
						VolumetricCloudCalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
						ConvertCloudPixelMaterialInputsToWorkingColorSpace(PixelMaterialInputs);
						OpticalDepth += SampleExtinctionCoefficients(PixelMaterialInputs) * ShadowLengthTest * CENTIMETER_TO_METER * DetlaNormT;
					}
#else
					// Linear shadow sample distribution.
					const float ShadowDtMeter = ShadowLengthTest * CENTIMETER_TO_METER / ShadowStepCount;
					const float ShadowJitteringSeed = float(ResolvedView.StateFrameIndexMod8) + PseudoRandom(SvPosition.xy) + i * 17;
					const float ShadowJitterNorm = InterleavedGradientNoise(SvPosition.xy, ShadowJitteringSeed) - 0.5f;
					for (float ShadowT = 0.5; ShadowT < ShadowStepCount; ShadowT += 1.0f)
					{
						const float ShadowSampleDistance = ShadowLengthTest * (ShadowT * InvShadowStepCount);
						UpdateMaterialCloudParam(MaterialParameters, SampleTranslatedWorldPosition + float3(0.0f, 0.0f, -1.0f) * ShadowSampleDistance, ShadowSampleWorldPositionDDX, ShadowSampleWorldPositionDDY,
							ResolvedView, CloudLayerParams, ShadowSampleDistance, SPACE_SKIPPING_SLICE_DEPTH_OFF);
#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY
						if (MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
						{
							continue; // Conservative density is 0 so skip and go to the next sample
						}
#endif
						VolumetricCloudCalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
						ConvertCloudPixelMaterialInputsToWorkingColorSpace(PixelMaterialInputs);
						OpticalDepth += SampleExtinctionCoefficients(PixelMaterialInputs) * ShadowDtMeter;
					}
#endif

					const float3 GroundBrdfNdotL = saturate(dot(Light0Direction, GroundNormal)) * (RenderVolumetricCloudParameters.GroundAlbedo.rgb / PI); // Assuming pure Lambert diffuse surface.
					const float3 GroundHemisphereLuminanceIsotropic = (2.0f * PI) * IsotropicPhase(); // Assumes the ground is uniform luminance to the cloud and solid angle is bottom hemisphere 2PI
					const float3 GroundToCloudTransfertIsoScatter = GroundBrdfNdotL * GroundHemisphereLuminanceIsotropic;

					const float3 AtmosphereTransmittanceToGround0 = AtmosphereTransmittanceToLight0; // big approximation when CLOUD_PER_SAMPLE_ATMOSPHERE_TRANSMITTANCE is true
					const float3 AtmosphereTransmittanceToGround1 = AtmosphereTransmittanceToLight1; // idem
					const float3 ScatteredLightLuminance = (AtmosphereTransmittanceToGround0 * Light0Illuminance + AtmosphereTransmittanceToGround1 * Light1Illuminance) * GroundToCloudTransfertIsoScatter;
					const float3 TransmittedScatteredLightLuminance = ScatteredLightLuminance * exp(-PMC.ExtinctionCoefficients[0] * MaterialParameters.CloudSampleAltitudeInLayer * CENTIMETER_TO_METER);

					DistantLightLuminance += ScatteredLightLuminance * exp(-OpticalDepth);
				}
#endif // MATERIAL_VOLUMETRIC_ADVANCED_GROUND_CONTRIBUTION

#if CLOUD_SAMPLE_ATMOSPHERIC_LIGHT_SHADOWMAP
				//
				// B- sample shadow from opaque if enabled. Only light0 supported.
				//
				bool bUnused;
				float OpaqueShadow = ComputeLight0VolumeShadowing(SampleTranslatedWorldPosition, false, false, bUnused);

#if VIRTUAL_SHADOW_MAP 
				if (RenderVolumetricCloudParameters.VirtualShadowMapId0 != INDEX_NONE)
				{
					FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapDirectional(RenderVolumetricCloudParameters.VirtualShadowMapId0, SampleTranslatedWorldPosition);
					OpaqueShadow *= VirtualShadowMapSample.ShadowFactor;
				}
#endif // VIRTUALSHADOW_MAP

				UNROLL
				for (ms = 0; ms < MSCOUNT; ++ms)
				{
					PMC.TransmittanceToLight0[ms] *= OpaqueShadow;
				}
#endif // CLOUD_SAMPLE_ATMOSPHERIC_LIGHT_SHADOWMAP

				//
				// C- shadow from volumetric LIGHT0
				//
				MATVEC ExtinctionAcc[MSCOUNT];
				const float ShadowLengthTest = RenderVolumetricCloudParameters.ShadowTracingMaxDistance;
				const float ShadowStepCount = float(RenderVolumetricCloudParameters.ShadowSampleCountMax);
				const float InvShadowStepCount = 1.0f / ShadowStepCount;
				const float ShadowJitteringSeed = float(ResolvedView.StateFrameIndexMod8) + PseudoRandom(SvPosition.xy);
				const float ShadowJitterNorm = 0.5f; // InterleavedGradientNoise(SvPosition.xy, ShadowJitteringSeed); // Disabled jittering for now as this one cannot be hidden well by TAA in some cases.
#if MATERIAL_VOLUMETRIC_ADVANCED_RAYMARCH_VOLUME_SHADOW==0 // Use cloud shadowmap instead of ray marching.
				float OutOpticalDepth = 0.0f;
				float CloudShadow = GetCloudVolumetricShadow(SampleTranslatedWorldPosition, RenderVolumetricCloudParameters.CloudShadowmapTranslatedWorldToLightClipMatrix[0], RenderVolumetricCloudParameters.CloudShadowmapFarDepthKm[0].x,
					RenderVolumetricCloudParameters.CloudShadowTexture0, RenderVolumetricCloudParameters.CloudBilinearTextureSampler, OutOpticalDepth);
				UNROLL
				for (ms = 0; ms < MSCOUNT; ++ms)
				{
					PMC.TransmittanceToLight0[ms] *= exp(-OutOpticalDepth * (ms == 0 ? 1.0f : pow(MsExtinFactor, ms)));
				}
#else			
				// Use raymarched shadows
				UNROLL
				for (ms = 0; ms < MSCOUNT; ++ms)
				{
					ExtinctionAcc[ms] = 0.0f;
				}
#if 0
				// Linear shadow samples (reference)
				const float ShadowDtMeter = ShadowLengthTest * CENTIMETER_TO_METER / ShadowStepCount;
				for (float ShadowT = ShadowJitterNorm; ShadowT < ShadowStepCount; ShadowT += 1.0f)
				{
					const float ShadowSampleDistance = ShadowLengthTest * (ShadowT * InvShadowStepCount);
					UpdateMaterialCloudParam(MaterialParameters, SampleTranslatedWorldPosition + Light0Direction * ShadowSampleDistance, ShadowSampleWorldPositionDDX, ShadowSampleWorldPositionDDY,
						ResolvedView, CloudLayerParams, ShadowSampleDistance, SPACE_SKIPPING_SLICE_DEPTH_OFF);
					const float ExtinctionFactor = 1.0f;
#else
				// Non-linear shadow sample distribution, hardcoded to x^2
				const float ShadowDtMeter = ShadowLengthTest * CENTIMETER_TO_METER;
				float PreviousNormT = 0.0f;
				for (float ShadowT = InvShadowStepCount; ShadowT <= 1.00001f; ShadowT += InvShadowStepCount)
				{
					float CurrentNormT = ShadowT * ShadowT; // CurrentNormT is the end of the considered segment to integrate, PreviousNormT is its beginning.
					const float DetlaNormT = CurrentNormT - PreviousNormT;
					const float ExtinctionFactor = DetlaNormT;
					const float ShadowSampleDistance = ShadowLengthTest * (PreviousNormT + DetlaNormT * ShadowJitterNorm);
					const float3 ShadowSampleTranslatedWorldPos = SampleTranslatedWorldPosition + Light0Direction * ShadowSampleDistance;
					UpdateMaterialCloudParam(MaterialParameters, ShadowSampleTranslatedWorldPos, ShadowSampleWorldPositionDDX, ShadowSampleWorldPositionDDY,
						ResolvedView, CloudLayerParams, ShadowSampleDistance, SPACE_SKIPPING_SLICE_DEPTH_OFF);
					PreviousNormT = CurrentNormT;
#endif

#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY
					if (MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
					{
					#if CLOUD_DEBUG_SAMPLES
						if (bDrawDebugEnabled) AddCrossWS(ShadowSampleTranslatedWorldPos - DFDemote(ResolvedView.PreViewTranslation), 30000.0f, float4(1, 0, 1, 0.5));
					#endif
						continue; // Conservative density is 0 so skip and go to the next sample
					}
					#if CLOUD_DEBUG_SAMPLES
					else
					{
						if (bDrawDebugEnabled) AddCrossWS(ShadowSampleTranslatedWorldPos - DFDemote(ResolvedView.PreViewTranslation), 30000.0f, float4(0, 1, 1, 0.5));
					}
					#endif // CLOUD_DEBUG_SAMPLES
#endif

					if (MaterialParameters.CloudSampleNormAltitudeInLayer <= 0.0f || MaterialParameters.CloudSampleNormAltitudeInLayer >= 1.0f)
					{
						break; // Ignore remaining samples since we have just traveled out of the cloud layer.
					}

					VolumetricCloudCalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
					ConvertCloudPixelMaterialInputsToWorkingColorSpace(PixelMaterialInputs);
					MATVEC ShadowExtinctionCoefficients = ColorToMATVEC(SampleExtinctionCoefficients(PixelMaterialInputs));

					ParticipatingMediaContext ShadowPMC = SetupParticipatingMediaContext(0.0f, ShadowExtinctionCoefficients, MsScattFactor, MsExtinFactor, 0.0f, 0.0f);

					UNROLL
					for (ms = 0; ms < MSCOUNT; ++ms)
					{
						ExtinctionAcc[ms] += ShadowPMC.ExtinctionCoefficients[ms] * ExtinctionFactor;
					}
				}

				UNROLL
				for (ms = 0; ms < MSCOUNT; ++ms)
				{
					PMC.TransmittanceToLight0[ms] *= exp(-ExtinctionAcc[ms] * ShadowDtMeter);
				}
#endif // Use cloud shadowmap

				//
				// C- shadow from volumetric LIGHT1
				//
#if CLOUD_SAMPLE_SECOND_LIGHT
				{ // CLOUD_SAMPLE_SECOND_LIGHT
#if MATERIAL_VOLUMETRIC_ADVANCED_RAYMARCH_VOLUME_SHADOW==0 // Use cloud shadowmap instead of ray marching.
					float OutOpticalDepth = 0.0f;
					float CloudShadow = GetCloudVolumetricShadow(SampleTranslatedWorldPosition, RenderVolumetricCloudParameters.CloudShadowmapTranslatedWorldToLightClipMatrix[1], RenderVolumetricCloudParameters.CloudShadowmapFarDepthKm[1].x,
						RenderVolumetricCloudParameters.CloudShadowTexture1, RenderVolumetricCloudParameters.CloudBilinearTextureSampler, OutOpticalDepth);
					UNROLL
					for (ms = 0; ms < MSCOUNT; ++ms)
					{
						PMC.TransmittanceToLight1[ms] *= exp(-OutOpticalDepth * (ms == 0 ? 1.0f : pow(MsExtinFactor, ms)));
					}
#else // Use cloud shadowmap
					UNROLL
					for (ms = 0; ms < MSCOUNT; ++ms)
					{
						ExtinctionAcc[ms] = 0.0f;
					}
					// Non-linear shadow sample distribution, hardcoded to x^2
					const float ShadowDtMeter = ShadowLengthTest * CENTIMETER_TO_METER;
					float PreviousNormT = 0.0f;
					for (float ShadowT = InvShadowStepCount; ShadowT <= 1.00001f; ShadowT += InvShadowStepCount)
					{
						float CurrentNormT = ShadowT * ShadowT; // CurrentNormT is the end of the considered segment to integrate, PreviousNormT is its beginning.
						const float DetlaNormT = CurrentNormT - PreviousNormT;
						const float ExtinctionFactor = DetlaNormT;
						const float ShadowSampleDistance = ShadowLengthTest * (PreviousNormT + DetlaNormT * ShadowJitterNorm);
						UpdateMaterialCloudParam(MaterialParameters, SampleTranslatedWorldPosition + Light1Direction * ShadowSampleDistance, ShadowSampleWorldPositionDDX, ShadowSampleWorldPositionDDY,
							ResolvedView, CloudLayerParams, ShadowSampleDistance, SPACE_SKIPPING_SLICE_DEPTH_OFF);
						PreviousNormT = CurrentNormT;

#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY
						if (MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
						{
							continue; // Conservative density is 0 so skip and go to the next sample
						}
#endif

						if (MaterialParameters.CloudSampleNormAltitudeInLayer <= 0.0f || MaterialParameters.CloudSampleNormAltitudeInLayer >= 1.0f)
						{
							break; // Ignore remaining samples since we have just traveld out of the cloud layer.
						}

						VolumetricCloudCalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
						ConvertCloudPixelMaterialInputsToWorkingColorSpace(PixelMaterialInputs);
						MATVEC ShadowExtinctionCoefficients = ColorToMATVEC(SampleExtinctionCoefficients(PixelMaterialInputs));

						ParticipatingMediaContext ShadowPMC = SetupParticipatingMediaContext(0.0f, ShadowExtinctionCoefficients, MsScattFactor, MsExtinFactor, 0.0f, 0.0f);

						UNROLL
						for (ms = 0; ms < MSCOUNT; ++ms)
						{
							ExtinctionAcc[ms] += ShadowPMC.ExtinctionCoefficients[ms] * ExtinctionFactor;
						}
					}

					UNROLL
					for (ms = 0; ms < MSCOUNT; ++ms)
					{
						PMC.TransmittanceToLight1[ms] *= exp(-ExtinctionAcc[ms] * ShadowDtMeter);
					}
#endif // Use cloud shadowmap
				} // CLOUD_SAMPLE_SECOND_LIGHT
#endif // CLOUD_SAMPLE_SECOND_LIGHT
				}


			//
			// From this point, MaterialParameters and CloudLayerParams cannot be used because they have been corrupted by the ray marched volume shadow integrator above!
			//


			// Compute the weighted average of t for the aerial perspective evaluation.
			if (any(PMC.ExtinctionCoefficients[0] > 0.0))
			{
				float tAPWeight = MATVEC_Min(TransmittanceToView);
				tAPWeightedSum += t * tAPWeight;
				tAPWeightsSum += tAPWeight;

			#if CLOUD_MIN_AND_MAX_DEPTH
				ClostestTAPWeightedSum += SampleContributeToClosest ? t * tAPWeight : 0.0f;
				ClostestTAPWeightsSum  += SampleContributeToClosest ?     tAPWeight : 0.0f;
			#endif // CLOUD_MIN_AND_MAX_DEPTH
			}


			//////////////////////////////
			// Evaluate local light in a slow way just so that it can be used for cinematics for now.
			//////////////////////////////
		#if CLOUD_SAMPLE_LOCAL_LIGHTS
			float3 LocalLightScatteredLuminance[MSCOUNT];
			UNROLL
			for (ms = 0; ms < MSCOUNT; ++ms)
			{
				LocalLightScatteredLuminance[ms] = 0.0f;
			}
			if (ForwardLightStruct.NumLocalLights > 0)
			{ 
				const float2 PixelPos = SvPosition.xy * RenderVolumetricCloudParameters.TracingCoordToFullResPixelCoordScaleBias.xy - ResolvedView.ViewRectMin.xy;// *RenderVolumetricCloudParameters.TracingCoordToFullResPixelCoordScaleBias.xy; // We do not take into account the bias so this owuld not work for split screens / VR.

				float4 ClipPos = mul(float4(SampleTranslatedWorldPosition, 1.0f), PrimaryView.TranslatedWorldToClip);
				ClipPos /= ClipPos.w;
				const float SceneDepth = ConvertFromDeviceZ(ClipPos.z);
				 
				uint GridIndex = ComputeLightGridCellIndex(PixelPos, SceneDepth, 0);
				const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);

				// This loop will process all unshadowed lights only.
				LOOP
				for (uint GridLightListIndex = 0; GridLightListIndex < CulledLightsGridHeader.NumLights; GridLightListIndex++)
				{
					const FLocalLightData LocalLight = GetLocalLightDataFromGrid(CulledLightsGridHeader.DataStartIndex + GridLightListIndex, 0);
			
					const float VolumetricScatteringIntensity = UnpackVolumetricScatteringIntensity(LocalLight);

					if (VolumetricScatteringIntensity > 0)
					{
						const FDeferredLightData LightData = ConvertToDeferredLight(LocalLight);

						float3 L = 0;
						float3 ToLight = 0;
						float LightMask = GetLocalLightAttenuation(SampleTranslatedWorldPosition, LightData, ToLight, L);

						float Lighting;
						if( LightData.bRectLight )
						{
							FRect Rect = GetRect( ToLight, LightData );
							Lighting = IntegrateLight(Rect);
						}
						else
						{
							FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
							Capsule.DistBiasSqr = 1.0f;
							Lighting = IntegrateLight(Capsule, LightData.bInverseSquared);
						}

						float3 CombinedAttenuation = Lighting * LightMask;
						if (CombinedAttenuation.r <= 0.0f)
						{
							continue;	// Skip shadow tracing if we know light is already not visible here
						}
						CombinedAttenuation *= LightData.Color;


						MATVEC ExtinctionAcc[MSCOUNT];
						const float SampleToLightLen = length(ToLight);
						const float3 SampleToLightNorm = ToLight / max(1e-5, SampleToLightLen);
						const float ShadowLengthTest = SampleToLightLen;
						const float ShadowStepCount = RenderVolumetricCloudParameters.LocalLightsShadowSampleCount;
						const float InvShadowStepCount = 1.0f / ShadowStepCount;
						const float ShadowJitterNorm = 0.5f;
						const float ShadowDtMeter = ShadowLengthTest * CENTIMETER_TO_METER / ShadowStepCount;

						UNROLL
						for (ms = 0; ms < MSCOUNT; ++ms)
						{
							ExtinctionAcc[ms] = 0.0f;
						}
						if (ShadowStepCount > 0.0f)
						{
							// Non-linear shadow sample distribution, hardcoded to x^2
							for (float ShadowT = ShadowJitterNorm; ShadowT < ShadowStepCount; ShadowT += 1.0f)
							{
								UpdateMaterialCloudParam(MaterialParameters, SampleTranslatedWorldPosition + SampleToLightNorm * ShadowLengthTest * (ShadowT * InvShadowStepCount), ShadowSampleWorldPositionDDX, ShadowSampleWorldPositionDDY,
									ResolvedView, CloudLayerParams, TRACING_SHADOW_DISTANCE_OFF, SPACE_SKIPPING_SLICE_DEPTH_OFF);
								const float ExtinctionFactor = 1.0f;

								VolumetricCloudCalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
								ConvertCloudPixelMaterialInputsToWorkingColorSpace(PixelMaterialInputs);
								MATVEC ShadowExtinctionCoefficients = ColorToMATVEC(SampleExtinctionCoefficients(PixelMaterialInputs));

								if (MaterialParameters.CloudSampleNormAltitudeInLayer <= 0.0f || MaterialParameters.CloudSampleNormAltitudeInLayer >= 1.0f)
								{
									ShadowExtinctionCoefficients = 0.0f; // Ignore remaining samples since we have just traveld out of the cloud layer.
								}

								ParticipatingMediaContext ShadowPMC = SetupParticipatingMediaContext(0.0f, ShadowExtinctionCoefficients, MsScattFactor, MsExtinFactor, 0.0f, 0.0f);

								UNROLL
								for (ms = 0; ms < MSCOUNT; ++ms)
								{
									ExtinctionAcc[ms] += ShadowPMC.ExtinctionCoefficients[ms] * ExtinctionFactor;
								}
							}
						}

						UNROLL
						for (ms = 0; ms < MSCOUNT; ++ms)
						{
							MATVEC LocalLightTransmittance = exp(-ExtinctionAcc[ms] * ShadowDtMeter);

							LocalLightScatteredLuminance[ms] += 
								VolumetricScatteringIntensity * CombinedAttenuation
								* IsotropicPhase()				// TODO add support for the cloud phase function
								* LocalLightTransmittance;		// Transmittance accounting for multiple scattering approximation
						}
					}
				}
			}
		#endif // CLOUD_SAMPLE_LOCAL_LIGHTS


			//////////////////////////////
			// Evaluate scattered luminance towards camera as well as view transmittance.
			//////////////////////////////
			UNROLL
			for (ms = MSCOUNT - 1; ms >= 0; --ms) // Must terminate with 0 because this is where TransmittanceToView is updated.
			{
				const MATVEC ScatteringCoefficients = PMC.ScatteringCoefficients[ms];
				const MATVEC ExtinctionCoefficients = PMC.ExtinctionCoefficients[ms];

				// Light 0
				const float3 TransmittanceToLight0 = PMC.TransmittanceToLight0[ms];
				float3 SunSkyLuminance = TransmittanceToLight0 * Light0IlluminanceFinal * PMPC.Phase0[ms];
				// Light 1
#if CLOUD_SAMPLE_SECOND_LIGHT
				const float3 TransmittanceToLight1 = PMC.TransmittanceToLight1[ms];
				SunSkyLuminance += TransmittanceToLight1 * Light1IlluminanceFinal * PMPC.Phase1[ms];
#endif
				// Distance sky light
				// *** The distance sky lighting contribution is ignored from the multi scattering approximation today because occlusion is not correclty handled (and as a result it would make clouds looks flat).
				// This could be removed when occlusion is better handled or approximated with a simple/expenssive optional  tracing.
				SunSkyLuminance += (ms == 0 ? DistantLightLuminance : float3(0.0f, 0.0f, 0.0f));

			#if CLOUD_SAMPLE_LOCAL_LIGHTS
				SunSkyLuminance += LocalLightScatteredLuminance[ms];
			#endif

				// *** EmissiveLuminance: it should be (EmissiveLuminance * AbsorptionCoefficients) but not intuitive for artist ==> can be added later as an option for consistency during path tracing. 
				// See "Production Volume Rendering", 2017, Section 2.2. So right now EmissiveLuminance is in fact LuminancePerMeter.
				// dt is not part of ScatteredLuminance because it is part of the analytical integral below.
				const float3 ScatteredLuminance = SunSkyLuminance * ScatteringCoefficients + EmissiveLuminance;

#if 0
				// Default iterative integration
				const MATVEC SafePathSegmentTransmittance = exp(-ExtinctionCoefficients * dtMeters);
				const float3 LuminanceContribution = TransmittanceToView * ScatteredLuminance * dtMeters;
				Luminance += LuminanceContribution;
				
			#if CLOUD_MIN_AND_MAX_DEPTH
				ClosestLuminance += SampleContributeToClosest ? LuminanceContribution : 0.0;
			#endif // CLOUD_MIN_AND_MAX_DEPTH

#elif 1
				// Improved scattering integration. See slide 28 of "Physically Based and Unified Volumetric Rendering in Frostbite"
				// Automatically works with emission. Emissive color is considered as a constant luminance emitted in all direction uniformly.
				MATVEC SafeExtinctionThreshold = 0.000001f;
				const MATVEC SafeExtinctionCoefficients = max(SafeExtinctionThreshold, ExtinctionCoefficients);
				const MATVEC SafePathSegmentTransmittance = exp(-SafeExtinctionCoefficients * dtMeters);
				float3 LuminanceIntegral = (ScatteredLuminance - ScatteredLuminance * SafePathSegmentTransmittance) / SafeExtinctionCoefficients;
				const float3 LuminanceContribution = TransmittanceToView * LuminanceIntegral;
				Luminance += LuminanceContribution;
				
			#if CLOUD_MIN_AND_MAX_DEPTH
				ClosestLuminance += SampleContributeToClosest ? LuminanceContribution : 0.0;
			#endif // CLOUD_MIN_AND_MAX_DEPTH

#else
				// Another integration that is not working nicely yet: explodes or need to use high extinction threshold 0.000001 to make it safe. And that has an impact on the final image.
				MATVEC S1 = TransmittanceToLight0 * ScatteringCoefficients;
				MATVEC SafeExtinctionThreshold = 0.000001f;
				const MATVEC SafeExtinctionCoefficients = max(SafeExtinctionThreshold, ExtinctionCoefficients);
				const MATVEC SafePathSegmentTransmittance = exp(-SafeExtinctionCoefficients * dtMeters);
				MATVEC Factor = SafePathSegmentTransmittance;
				float3 AnalyticalShadowedScattered = (Factor * S0 - Factor * S1 - Factor * dtMeters * SafeExtinctionCoefficients * S1 + ((-float3(1.0, 1.0, 1.0) + dtMeters * SafeExtinctionCoefficients) * S0 + S1))
					/ (dtMeters * SafeExtinctionCoefficients * SafeExtinctionCoefficients);

				/// Not handled yet: second light and emissive 
				float3 SkyScatteredLuminance = SunSkyLuminance * ScatteringCoefficients;
				float3 SkyLuminanceIntegral = (ScatteredLuminance - ScatteredLuminance * SafePathSegmentTransmittance) / SafeExtinctionCoefficients;
				const float3 LuminanceContribution = TransmittanceToView * ((AnalyticalShadowedScattered * Light0IlluminanceFinal * PMPC.Phase0[ms]) + (ms == 0 ? SkyLuminanceIntegral : float3(0.0f, 0.0f, 0.0f)));
				Luminance += LuminanceContribution;
				if (ms == 0) S0 = S1;
				
			#if CLOUD_MIN_AND_MAX_DEPTH
				ClosestLuminance += SampleContributeToClosest ? LuminanceContribution : 0.0;
			#endif // CLOUD_MIN_AND_MAX_DEPTH
#endif

				if (ms == 0)
				{
					TransmittanceToView *= SafePathSegmentTransmittance;

				#if CLOUD_MIN_AND_MAX_DEPTH
					ClosestTransmittanceToView *= TClosestDepthBufferCm > t ? SafePathSegmentTransmittance : 1.0;
				#endif // CLOUD_MIN_AND_MAX_DEPTH
				}
			}

			// This is helpful for performance. Can result in less light pucnhing through clouds. Should be a setting.
			if (all(TransmittanceToView < RenderVolumetricCloudParameters.StopTracingTransmittanceThreshold) && (!CLOUD_DEBUG_VIEW_MODE))
			{
				break;
			}

			t += StepT;
		}

#if CLOUD_DEBUG_VIEW_MODE
		if (RenderVolumetricCloudParameters.CloudDebugViewMode >= 1)
		{
			OutColor0.r = 1.0 - saturate(DebugZeroConservativeDensitySampleCount / float(IStepCount));
		#if SHADER_RENDERVIEW_CS_WITH_EMPTY_SPACE_SKIPPING
			if (RenderVolumetricCloudParameters.CloudDebugViewMode == 2)
			{
				OutColor0.g = saturate(StartDepthCm / StartTracingSampleVolumeDepth);
			}
		#endif
		}
#endif
	}
	////////////////////////////////////////////////////////////////////////////////////////////////////
	// End of cloud ray marching
	////////////////////////////////////////////////////////////////////////////////////////////////////



	// When rendering a real time reflection capture (sky envmap) whe use a different exposure
	const float OutputPreExposure = (ResolvedView.RealTimeReflectionCapture ? ResolvedView.RealTimeReflectionCapturePreExposure : ResolvedView.PreExposure);

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (IsVolumetricCloudHoldout(View.EnvironmentComponentsFlags) && View.bPrimitiveAlphaHoldoutEnabled)
	{
		Luminance *= 0;
	#if CLOUD_MIN_AND_MAX_DEPTH
		ClosestLuminance *= 0;
	#endif

		OutHoldOutCoverage = saturate(1.0f - MATVEC_Avg(TransmittanceToView));
	}
#endif

	//
	// Apply aerial perspective if needed
	//

	// This is the default depth when no cloud has been intersected. 
	// It is better to limit depth to a close distance instead of max float to smooth out cloud edges when intersecting opaque meshes.
	// No visual issues have been noticed with reprojection+TAA so far.
	const float NoCloudDepth = TMax;
	const float tAP = tAPWeightsSum==0.0f ? NoCloudDepth : tAPWeightedSum / max(0.0000000001f, tAPWeightsSum);
	float MeanTransmittance = MATVEC_Avg(TransmittanceToView);
#if CLOUD_MIN_AND_MAX_DEPTH
	const float ClosestTAP = ClostestTAPWeightsSum == 0.0f ? NoCloudDepth : ClostestTAPWeightedSum / max(0.0000000001f, ClostestTAPWeightsSum);
	float ClosestMeanTransmittance = MATVEC_Avg(ClosestTransmittanceToView);
#endif // CLOUD_MIN_AND_MAX

	float3 FogSampleWorldPositionRelativeToCameraCm = tAP * Raydir;
	float4 ClipPos = mul(float4(RayTranslatedWorldOriginKm * KILOMETER_TO_CENTIMETER + FogSampleWorldPositionRelativeToCameraCm, 1.0f), PrimaryView.TranslatedWorldToClip);

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	const bool bApplyFogOnCloud			= RenderVolumetricCloudParameters.EnableHeightFog >= 2 || tAPWeightsSum;					// Use MeanTransmittance instead of tAPWeightsSum as it is more representative of what can happen since it has not been transformed. (but can result in more fog being applied, this makes alpha holdout correct)
#if CLOUD_MIN_AND_MAX_DEPTH
	const bool bApplyFogOnClosestCloud	= RenderVolumetricCloudParameters.EnableHeightFog >= 2 || ClosestMeanTransmittance < 1.0f;	// Idem
#endif
#else // SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	// We skip more pixels when using tAPWeightsSum so let;s keep that when alpha output is not enabled for the sake of performance
	const bool bApplyFogOnCloud			= RenderVolumetricCloudParameters.EnableHeightFog >= 2 || tAPWeightsSum > 0.0f;
#if CLOUD_MIN_AND_MAX_DEPTH
	const bool bApplyFogOnClosestCloud	= RenderVolumetricCloudParameters.EnableHeightFog >= 2 || ClostestTAPWeightsSum > 0.0f;
#endif
#endif // SUPPORT_PRIMITIVE_ALPHA_HOLDOUT

	if (RenderVolumetricCloudParameters.EnableAerialPerspectiveSampling && bApplyFogOnCloud)
	{
		float4 AerialPerspective;

		if (!ResolvedView.RealTimeReflectionCapture &&	// We do not generate a 360 CameraAerialPerspectiveVolume with separated Mie and Ray contribution as of today.
			(RenderVolumetricCloudParameters.AerialPerspectiveRayOnlyStartDistanceKm + RenderVolumetricCloudParameters.AerialPerspectiveMieOnlyStartDistanceKm) > 0.0f)
		{
			float4 MieAP = GETAERIALPERSPECTIVE(View.CameraAerialPerspectiveVolumeMieOnly, View.CameraAerialPerspectiveVolumeMieOnlySampler, FogSampleWorldPositionRelativeToCameraCm,
				max(ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm, RenderVolumetricCloudParameters.AerialPerspectiveMieOnlyStartDistanceKm), RenderVolumetricCloudParameters.AerialPerspectiveMieOnlyFadeDistanceKmInv);
			float4 RayAP = GETAERIALPERSPECTIVE(View.CameraAerialPerspectiveVolumeRayOnly, View.CameraAerialPerspectiveVolumeRayOnlySampler, FogSampleWorldPositionRelativeToCameraCm,
				max(ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm, RenderVolumetricCloudParameters.AerialPerspectiveRayOnlyStartDistanceKm), RenderVolumetricCloudParameters.AerialPerspectiveRayOnlyFadeDistanceKmInv);

			AerialPerspective = float4(MieAP.rgb + RayAP.rgb, MieAP.a * RayAP.a);
		}
		else
		{
			const float NearFadeOutRangeInvDepthKm = 1.0 / 0.00001f; // 1 centimeter fade region
			// Apply AP only once according to the mean position within the participating media weighted by transmittance/visibility.
			// This allows to apply AP only once per pixel instead of per sample.
			AerialPerspective = GETAERIALPERSPECTIVE(View.CameraAerialPerspectiveVolume, View.CameraAerialPerspectiveVolumeSampler, FogSampleWorldPositionRelativeToCameraCm, ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm, NearFadeOutRangeInvDepthKm);
		}

		// Apply aerial perspective OVER the cloud, assiming coverage is from the full transmittance.
		// skip if MeanTransmittance is 1
		float CloudMeanCoverage = 1.0 - MeanTransmittance;
		Luminance = AerialPerspective.rgb * CloudMeanCoverage + AerialPerspective.a * Luminance;

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
		if(View.bPrimitiveAlphaHoldoutEnabled)
		{
			const float AerialPespectiveTransmittance = AerialPerspective.a;
			const float AerialPespectiveCoverage = saturate(1.0 - AerialPespectiveTransmittance);

			// Fade out holdout coverage as a function of the aerial perspective above, if the sky atmosphere is not hold out
			OutHoldOutCoverage   *= AerialPespectiveTransmittance;

			if(IsSkyAtmosphereHoldout(View.EnvironmentComponentsFlags))
			{
				OutAboveCloudHoldOut = OutAboveCloudHoldOut * AerialPespectiveTransmittance + CloudMeanCoverage * AerialPespectiveCoverage;
			}
		}
#endif
	}


	//
	// Apply height fog if needed
	//
	if (bApplyFogOnCloud)
	{
		float4 HeightFogInscatteringAndTransmittance = float4(0, 0, 0, 1);

		if (RenderVolumetricCloudParameters.EnableHeightFog > 0)
		{
			HeightFogInscatteringAndTransmittance = CalculateHeightFog(FogSampleWorldPositionRelativeToCameraCm);
		}

	#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
		uint2 TilePos = uint2(SvPosition.xy * RenderVolumetricCloudParameters.TracingCoordToFullResPixelCoordScaleBias.xy - View.ViewRectMin.xy) / LFVTilePixelSize.xx;
		float4 LFVContribution = GetLFVContribution(PrimaryView, TilePos, FogSampleWorldPositionRelativeToCameraCm);
		if (LFVRenderInVolumetricFog > 0)
		{
			HeightFogInscatteringAndTransmittance = float4(LFVContribution.rgb + HeightFogInscatteringAndTransmittance.rgb * LFVContribution.a, LFVContribution.a * HeightFogInscatteringAndTransmittance.a);
		}
	#endif

		if (FogStruct.ApplyVolumetricFog > 0)
		{
			float3 VolumeUV = ComputeVolumeUVFromNDC(ClipPos);
			const uint EyeIndex = 0;
			HeightFogInscatteringAndTransmittance = CombineVolumetricFog(HeightFogInscatteringAndTransmittance, VolumeUV, EyeIndex, length(FogSampleWorldPositionRelativeToCameraCm));
		}

	#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
		if (LFVRenderInVolumetricFog == 0)
		{
			HeightFogInscatteringAndTransmittance = float4(LFVContribution.rgb + HeightFogInscatteringAndTransmittance.rgb * LFVContribution.a, LFVContribution.a * HeightFogInscatteringAndTransmittance.a);
		}
	#endif

		// Apply aerial perspective OVER the cloud, assiming coverage is from the full transmittance.
		// skip if MeanTransmittance is 1
		float CloudMeanCoverage = 1.0 - MeanTransmittance;
		Luminance = HeightFogInscatteringAndTransmittance.rgb * CloudMeanCoverage + HeightFogInscatteringAndTransmittance.a * Luminance;

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
		if(View.bPrimitiveAlphaHoldoutEnabled)
		{
			const float ExponentialFogTransmittance = HeightFogInscatteringAndTransmittance.a;
			const float ExponentialFogCoverage = saturate(1.0 - ExponentialFogTransmittance);

			// Fade out holdout coverage as a function of the fog above, if the fog is not hold out
			OutHoldOutCoverage *= ExponentialFogTransmittance;

			if(IsExponentialFogHoldout(View.EnvironmentComponentsFlags))
			{
				OutAboveCloudHoldOut = OutAboveCloudHoldOut * ExponentialFogTransmittance + CloudMeanCoverage * ExponentialFogCoverage;
			}
		}
#endif
	}


	// Apply atmosphere and fog on closest if needed
#if CLOUD_MIN_AND_MAX_DEPTH
	{
		float3 FogSampleWorldPositionRelativeToCameraCm = ClosestTAP * Raydir;
		float4 ClipPos = mul(float4(RayTranslatedWorldOriginKm * KILOMETER_TO_CENTIMETER + FogSampleWorldPositionRelativeToCameraCm, 1.0f), PrimaryView.TranslatedWorldToClip);

		if (RenderVolumetricCloudParameters.EnableAerialPerspectiveSampling && bApplyFogOnClosestCloud)
		{
			float4 AerialPerspective;

			if (!ResolvedView.RealTimeReflectionCapture &&	// We do not generate a 360 CameraAerialPerspectiveVolume with separated Mie and Ray contribution as of today.
				(RenderVolumetricCloudParameters.AerialPerspectiveRayOnlyStartDistanceKm + RenderVolumetricCloudParameters.AerialPerspectiveMieOnlyStartDistanceKm) > 0.0f)
			{
				float4 MieAP = GETAERIALPERSPECTIVE(View.CameraAerialPerspectiveVolumeMieOnly, View.CameraAerialPerspectiveVolumeMieOnlySampler, FogSampleWorldPositionRelativeToCameraCm,
					max(ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm, RenderVolumetricCloudParameters.AerialPerspectiveMieOnlyStartDistanceKm), RenderVolumetricCloudParameters.AerialPerspectiveMieOnlyFadeDistanceKmInv);
				float4 RayAP = GETAERIALPERSPECTIVE(View.CameraAerialPerspectiveVolumeRayOnly, View.CameraAerialPerspectiveVolumeRayOnlySampler, FogSampleWorldPositionRelativeToCameraCm,
					max(ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm, RenderVolumetricCloudParameters.AerialPerspectiveRayOnlyStartDistanceKm), RenderVolumetricCloudParameters.AerialPerspectiveRayOnlyFadeDistanceKmInv);

				AerialPerspective = float4(MieAP.rgb + RayAP.rgb, MieAP.a * RayAP.a);
			}
			else
			{
				const float NearFadeOutRangeInvDepthKm = 1.0 / 0.00001f; // 1 centimeter fade region
				// Apply AP only once according to the mean position within the participating media weighted by transmittance/visibility.
				// This allows to apply AP only once per pixel instead of per sample.
				AerialPerspective = GETAERIALPERSPECTIVE(View.CameraAerialPerspectiveVolume, View.CameraAerialPerspectiveVolumeSampler, FogSampleWorldPositionRelativeToCameraCm, ResolvedView.SkyAtmosphereAerialPerspectiveStartDepthKm, NearFadeOutRangeInvDepthKm);
			}

			// Apply aerial perspective OVER the cloud, assiming coverage is from the full transmittance.
			// skip if ClosestMeanTransmittance is 1
			float MeanCoverage = 1.0 - ClosestMeanTransmittance;
			ClosestLuminance = AerialPerspective.rgb * MeanCoverage + AerialPerspective.a * ClosestLuminance;
		}


		//
		// Apply height fog if needed
		//
		if (bApplyFogOnClosestCloud)
		{
			float4 HeightFogInscatteringAndTransmittance = float4(0, 0, 0, 1);

			if (RenderVolumetricCloudParameters.EnableHeightFog > 0)
			{
				HeightFogInscatteringAndTransmittance = CalculateHeightFog(FogSampleWorldPositionRelativeToCameraCm);
			}

		#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
			uint2 TilePos = uint2(SvPosition.xy * RenderVolumetricCloudParameters.TracingCoordToFullResPixelCoordScaleBias.xy - View.ViewRectMin.xy) / LFVTilePixelSize.xx;
			float4 LFVContribution = GetLFVContribution(PrimaryView, TilePos, FogSampleWorldPositionRelativeToCameraCm);
			if (LFVRenderInVolumetricFog > 0)
			{
				HeightFogInscatteringAndTransmittance = float4(LFVContribution.rgb + HeightFogInscatteringAndTransmittance.rgb * LFVContribution.a, LFVContribution.a * HeightFogInscatteringAndTransmittance.a);
			}
		#endif

			if (FogStruct.ApplyVolumetricFog > 0)
			{
				float3 VolumeUV = ComputeVolumeUVFromNDC(ClipPos);
				const uint EyeIndex = 0;
				HeightFogInscatteringAndTransmittance = CombineVolumetricFog(HeightFogInscatteringAndTransmittance, VolumeUV, EyeIndex, length(FogSampleWorldPositionRelativeToCameraCm));
			}

		#if PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME
			if (LFVRenderInVolumetricFog == 0)
			{
				HeightFogInscatteringAndTransmittance = float4(LFVContribution.rgb + HeightFogInscatteringAndTransmittance.rgb * LFVContribution.a, LFVContribution.a * HeightFogInscatteringAndTransmittance.a);
			}
		#endif

			// Apply aerial perspective OVER the cloud, assiming coverage is from the full transmittance.
			// skip if MeanTransmittance is 1
			float MeanCoverage = 1.0 - ClosestMeanTransmittance;
			ClosestLuminance = HeightFogInscatteringAndTransmittance.rgb * MeanCoverage + HeightFogInscatteringAndTransmittance.a * ClosestLuminance;
		}
	}
#endif // CLOUD_MIN_AND_MAX_DEPTH

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback, SvPosition, View.VTFeedbackBuffer);
#endif

	//
	// Output result
	//

	float GrayScaleTransmittance = MeanTransmittance < RenderVolumetricCloudParameters.StopTracingTransmittanceThreshold ? 0.0f : MeanTransmittance;
#if CLOUD_DEBUG_VIEW_MODE
	if (RenderVolumetricCloudParameters.CloudDebugViewMode == 1)
	{
		OutColor0 = float4(OutColor0.r * OutputPreExposure, Luminance.gb * OutputPreExposure, GrayScaleTransmittance);
	}
	else if (RenderVolumetricCloudParameters.CloudDebugViewMode == 2)
	{
		OutColor0 = float4(OutColor0.rg * OutputPreExposure, Luminance.b * OutputPreExposure, GrayScaleTransmittance);
	}
#else
	OutColor0 = float4(Luminance * OutputPreExposure, GrayScaleTransmittance);
#endif

#if CLOUD_MIN_AND_MAX_DEPTH
#if CLOUD_DEBUG_VIEW_MODE
	OutColor1 = OutColor0;
#else
	float ClosestGrayScaleTransmittance = ClosestMeanTransmittance < RenderVolumetricCloudParameters.StopTracingTransmittanceThreshold ? 0.0f : ClosestMeanTransmittance;
	OutColor1 = float4(ClosestLuminance * OutputPreExposure, ClosestGrayScaleTransmittance);
#endif
#endif // CLOUD_MIN_AND_MAX

	OutDepth.x = MaxHalfFloat; // Default to far away depth to be flat and not intersect with any geometry.
	//if (RenderVolumetricCloudParameters.OpaqueIntersectionMode >= 1) => Always __XB_GDS_Write_U32 depth as this helps with reprojection in all modes, also mode 2 where this helps reduce latency.
	{
		OutDepth.x = ((GrayScaleTransmittance > 0.99) ? NoCloudDepth : tAP) * CENTIMETER_TO_KILOMETER; // using a small threshold on transmittance
	}

	//OutColor0.g = 0.3f; // Debug revealing reveal pixels that have been traced.
}

#endif // SHADER_RENDERVIEW_PS || SHADER_RENDERVIEW_CS

#if SHADER_RENDERVIEW_PS

void MainPS(
	in float4 SvPosition : SV_Position
	, out float4 OutColor0 : SV_Target0
	, out float4 OutDepth : SV_Target1)
{
	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
	
	float4 OutDepth4 = 0;
	float OutHoldOutCoverage = 0.0f;
	float OutAboveCloudHoldOut = 0.0f;
	MainCommon(MaterialParameters, SvPosition, OutColor0, OutDepth4, OutHoldOutCoverage, OutAboveCloudHoldOut);
	OutDepth = float4(OutDepth4.xy, 0.0f, 0.0f);	// PS path never writes min and max tracing depth in zw.
}

#endif // SHADER_RENDERVIEW_PS

#if SHADER_RENDERVIEW_CS

float4 OutputViewRect;
int bBlendCloudColor;
int TargetCubeFace;
int bAccumulateAlphaHoldOut;

RWTexture2D<float4> OutCloudColor0;
#if CLOUD_MIN_AND_MAX_DEPTH
RWTexture2D<float4> OutCloudColor1;
#endif
RWTexture2D<float4> OutCloudDepth;
RWTextureCube<float4> OutCloudColorCube;
#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
RWTexture2D<float> OutCloudAlphaHoldout;
#endif

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, 1)]
void MainCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	BRANCH
	if (all(DispatchThreadId < uint2(OutputViewRect.zw)))
	{
		float4 SvPosition = float4(DispatchThreadId.x + 0.5, DispatchThreadId.y + 0.5, 0.5, 1);
		float4 OutColor0;
#if CLOUD_MIN_AND_MAX_DEPTH
		float4 OutColor1;
#endif
		float4 OutDepth;

		FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();

		float OutHoldOutCoverage = 0.0f;
		float OutAboveCloudHoldOut = 0.0f;
		MainCommon(MaterialParameters, SvPosition, OutColor0
#if CLOUD_MIN_AND_MAX_DEPTH
			, OutColor1
#endif
			, OutDepth
			, OutHoldOutCoverage
			, OutAboveCloudHoldOut);

		if (TargetCubeFace >= 0)
		{
			uint3 CubemapCoord = uint3(DispatchThreadId, TargetCubeFace);
			if (bBlendCloudColor)
			{
				float4 CurColor0 = OutCloudColorCube[CubemapCoord];
				OutCloudColorCube[CubemapCoord] = float4(CurColor0.rgb * OutColor0.a + OutColor0.rgb, CurColor0.a * OutColor0.a);
			}
			else
			{
				OutCloudColorCube[CubemapCoord] = OutColor0;
			}
		}
		else
		{
			if (bBlendCloudColor)
			{
				float4 CurColor0 = OutCloudColor0[DispatchThreadId];
				OutCloudColor0[DispatchThreadId] = float4(CurColor0.rgb * OutColor0.a + OutColor0.rgb, CurColor0.a);
				// No alpha holdout supported with blening.
			}
			else
			{
				OutCloudColor0[DispatchThreadId] = OutColor0;
			#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
				if (View.bPrimitiveAlphaHoldoutEnabled)
				{
					if (IsVolumetricCloudHoldout(View.EnvironmentComponentsFlags) && bAccumulateAlphaHoldOut > 0)
					{
						OutCloudAlphaHoldout[DispatchThreadId] = OutHoldOutCoverage + OutAboveCloudHoldOut;
					}
					else
					{
						OutCloudAlphaHoldout[DispatchThreadId] = OutAboveCloudHoldOut;
					}
				}
			#endif
			}
		}
#if CLOUD_MIN_AND_MAX_DEPTH
		OutCloudColor1[DispatchThreadId] = OutColor1;
#endif
		OutCloudDepth[DispatchThreadId] = OutDepth;
	}
}

#endif // SHADER_RENDERVIEW_CS



#if SHADER_EMPTY_SPACE_SKIPPING_CS

float2	StartTracingDistanceTextureResolution;
float	StartTracingSampleVolumeDepth;
float	StartTracingSliceBias;

RWTexture2D<float> OutStartTracingDistanceTexture;

struct FThreadData
{
	float FrontDepthKm;
};
groupshared FThreadData GroupData[THREADGROUP_SIZEX][THREADGROUP_SIZEY][THREADGROUP_SIZEZ];

float GetDefaultDepth()
{
	return StartTracingSampleVolumeDepth * CENTIMETER_TO_KILOMETER;
}

FThreadData GetDefaultThreadData()
{
	FThreadData ThreadData;
	ThreadData.FrontDepthKm = GetDefaultDepth();
	return ThreadData;
}

FThreadData Reduce(FThreadData A, FThreadData B)
{
	FThreadData Out;
	Out.FrontDepthKm = min(A.FrontDepthKm, B.FrontDepthKm);
	return Out;
}

float GetSliceDepth()
{
	return RenderVolumetricCloudParameters.EmptySpaceSkippingSliceDepth;
}

float GetSvPositionEmptySpaceSkippingValue(in ViewState ResolvedView, in uint3 DispatchThreadId, in uint3 GroupThreadId, in float3 SampleFroxelOffset)
{
#if MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT
	float2 PixelUV = float2(DispatchThreadId.x + SampleFroxelOffset.x, DispatchThreadId.y + SampleFroxelOffset.y) / StartTracingDistanceTextureResolution;
	float4 SvPosition = float4(PixelUV * ResolvedView.ViewSizeAndInvSize.xy, 0.1, 1);

	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
	
	// CalcMaterialParameters() evaluates the material, but since the depth associated with SvPosition at this point doesn't correspond to any particular point along the ray,
	// we leave WorldPosition_DDX/Y as 0, which forces mip 0 when compiled with USE_ANALYTIC_DERIVATIVES.
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, true);

	FCloudLayerParameters CloudLayerParams = GetCloudLayerParams(
		RenderVolumetricCloudParameters.CloudLayerCenterKm, RenderVolumetricCloudParameters.PlanetRadiusKm,
		RenderVolumetricCloudParameters.BottomRadiusKm, RenderVolumetricCloudParameters.TopRadiusKm);

	const float SampleDistanceFromCamera = (float(GroupThreadId.z) + SampleFroxelOffset.z) * GetSliceDepth();
	const float3 Raydir = -MaterialParameters.CameraVector;
	const float3 SampleTranslatedWorldPosition = GetTranslatedWorldCameraPosFromView(ResolvedView, SvPosition.xy) + Raydir * SampleDistanceFromCamera;

#if USE_ANALYTIC_DERIVATIVES
	const float3 TranslatedWorldPositionOffsetX = SvPositionToResolvedTranslatedWorld(SvPosition + float4(1.0f, 0.0f, 0.0f, 0.0f));
	const float3 TranslatedWorldPositionOffsetY = SvPositionToResolvedTranslatedWorld(SvPosition + float4(0.0f, 1.0f, 0.0f, 0.0f));
	const float3 RayDirOffsetX = GetCameraVector(ResolvedView, TranslatedWorldPositionOffsetX, ResolvedView.TranslatedWorldCameraOrigin);
	const float3 RayDirOffsetY = GetCameraVector(ResolvedView, TranslatedWorldPositionOffsetY, ResolvedView.TranslatedWorldCameraOrigin);
	float3 SampleWorldPositionDDX = (GetTranslatedWorldCameraPosFromView(ResolvedView, SvPosition.xy + float2(1.0f, 0.0f)) + RayDirOffsetX * SampleDistanceFromCamera) - SampleTranslatedWorldPosition;
	float3 SampleWorldPositionDDY = (GetTranslatedWorldCameraPosFromView(ResolvedView, SvPosition.xy + float2(0.0f, 1.0f)) + RayDirOffsetY * SampleDistanceFromCamera) - SampleTranslatedWorldPosition;
#else
	float3 SampleWorldPositionDDX = 0.0f;
	float3 SampleWorldPositionDDY = 0.0f;
#endif // USE_ANALYTIC_DERIVATIVES

	UpdateMaterialCloudParam(MaterialParameters, SampleTranslatedWorldPosition, SampleWorldPositionDDX, SampleWorldPositionDDY,
		ResolvedView, CloudLayerParams, TRACING_SHADOW_DISTANCE_OFF, GetSliceDepth());

	const float CloudEmptySpaceSkippingOutput0 = GetVolumetricCloudEmptySpaceSkippingOutput0(MaterialParameters);
	return CloudEmptySpaceSkippingOutput0;
#else
	return 0;
#endif
}

[numthreads(THREADGROUP_SIZEX, THREADGROUP_SIZEY, THREADGROUP_SIZEZ)]
void MainCS(uint3 DispatchThreadId : SV_DispatchThreadID, uint3 GroupThreadId : SV_GroupThreadID)
{
#if MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT

	ResolvedView = ResolveView();

	float CloudEmptySpaceSkippingOutput0 = 0.0f;

	// Center samples
	{
		const float3 SampleFroxelOffset = float3(0.5f, 0.5f, 0.5f);
		CloudEmptySpaceSkippingOutput0 = max(CloudEmptySpaceSkippingOutput0, GetSvPositionEmptySpaceSkippingValue(ResolvedView, DispatchThreadId, GroupThreadId, SampleFroxelOffset));
	}

#if EMPTY_SPACE_SKIPPING_SAMPLE_CORNERS

	// Corner samples
	UNROLL
	for(float x=0; x<=1; ++x)
	{
		UNROLL
		for(float y=0; y<=1; ++y)
		{
			UNROLL
			for(float z=0; z<=1; ++z)
			{
				CloudEmptySpaceSkippingOutput0 = max(CloudEmptySpaceSkippingOutput0, GetSvPositionEmptySpaceSkippingValue(ResolvedView, DispatchThreadId, GroupThreadId, float3(x, y, z)));
			}
		}
	}

#endif

	GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z].FrontDepthKm = max(0.0f, CloudEmptySpaceSkippingOutput0 > 0.0f ? (float(GroupThreadId.z) + StartTracingSliceBias) * GetSliceDepth() * CENTIMETER_TO_KILOMETER : GetDefaultDepth());

	GroupMemoryBarrierWithGroupSync();

	// Reduction
#if THREADGROUP_SIZEZ > 64
#error Reduction does not support THREADGROUP_SIZEZ > 64
#endif

#if THREADGROUP_SIZEZ >= 64
	if (GroupThreadId.z < 32)
	{
		GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z] = Reduce(
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z],
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z + 32]);
	}
	GroupMemoryBarrierWithGroupSync();
#endif

#if THREADGROUP_SIZEZ >= 32
	if (GroupThreadId.z < 16)
	{
		GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z] = Reduce(
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z],
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z + 16]);
	}
	GroupMemoryBarrierWithGroupSync();
#endif

#if THREADGROUP_SIZEZ >= 16
	if (GroupThreadId.z < 8)
	{
		GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z] = Reduce(
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z],
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z + 8]);
	}
	GroupMemoryBarrierWithGroupSync();
#endif

#if THREADGROUP_SIZEZ >= 8
	if (GroupThreadId.z < 4)
	{
		GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z] = Reduce(
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z],
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z + 4]);
	}
	GroupMemoryBarrierWithGroupSync();
#endif

	// The smallest wave size is 4 on Mali G-71 hardware. So now we can do simple math operations without group sync.

#if THREADGROUP_SIZEZ >= 4
	if (GroupThreadId.z < 2)
	{
		GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z] = Reduce(
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z],
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z + 2]);
	}
#endif

	if (GroupThreadId.z < 1)
	{
		GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z] = Reduce(
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z],
			GroupData[GroupThreadId.x][GroupThreadId.y][GroupThreadId.z + 1]);

		BRANCH
		if (all(DispatchThreadId.xy < uint2(StartTracingDistanceTextureResolution.xy)))
		{
			OutStartTracingDistanceTexture[DispatchThreadId.xy] = GroupData[GroupThreadId.x][GroupThreadId.y][0].FrontDepthKm;
		}
	}

#else // MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT

	OutStartTracingDistanceTexture[DispatchThreadId.xy] = 0.0f;	// No empty space skipping

#endif // MATERIAL_VOLUMETRIC_CLOUD_EMPTY_SPACE_SKIPPING_OUTPUT
}

#endif // SHADER_EMPTY_SPACE_SKIPPING_CS



#if SHADER_SHADOW_PS

struct CloudShadowTraceContext
{
	float		FarDepthKm;
	float		Strength;
	float		DepthBias;
	float		SampleCount;
	float4		SizeInvSize;
	float4		TracingPixelScaleOffset;
	float4		TracingSizeInvSize;
	float4x4	TranslatedWorldToLightClipMatrix;
	float4x4	TranslatedWorldToLightClipMatrixInv;
	float3		TraceDir;
};

void MainPS(
	in float4 SvPosition : SV_Position
	, out float3 OutColor0 : SV_Target0
)
{
	ResolvedView = ResolveView();

	CloudShadowTraceContext TraceContext;
	if (RenderVolumetricCloudParameters.TraceShadowmapMode >=1)
	{
		const uint LightIndex = clamp(RenderVolumetricCloudParameters.TraceShadowmapMode - 1, 0, 1);
		TraceContext.FarDepthKm					= RenderVolumetricCloudParameters.CloudShadowmapFarDepthKm[LightIndex].x;
		TraceContext.Strength					= RenderVolumetricCloudParameters.CloudShadowmapStrength[LightIndex].x;
		TraceContext.DepthBias					= RenderVolumetricCloudParameters.CloudShadowmapDepthBias[LightIndex].x;
		TraceContext.SampleCount				= RenderVolumetricCloudParameters.CloudShadowmapSampleCount[LightIndex].x;
		TraceContext.SizeInvSize				= RenderVolumetricCloudParameters.CloudShadowmapSizeInvSize[LightIndex];
		TraceContext.TracingSizeInvSize			= RenderVolumetricCloudParameters.CloudShadowmapTracingSizeInvSize[LightIndex];
		TraceContext.TracingPixelScaleOffset	= RenderVolumetricCloudParameters.CloudShadowmapTracingPixelScaleOffset[LightIndex];
		TraceContext.TranslatedWorldToLightClipMatrix		= RenderVolumetricCloudParameters.CloudShadowmapTranslatedWorldToLightClipMatrix[LightIndex];
		TraceContext.TranslatedWorldToLightClipMatrixInv	= RenderVolumetricCloudParameters.CloudShadowmapTranslatedWorldToLightClipMatrixInv[LightIndex];
		TraceContext.TraceDir					= RenderVolumetricCloudParameters.CloudShadowmapLightDir[LightIndex].xyz;
	}
	else
	{
		TraceContext.FarDepthKm					= RenderVolumetricCloudParameters.CloudSkyAOFarDepthKm;
		TraceContext.Strength					= RenderVolumetricCloudParameters.CloudSkyAOStrength;
		TraceContext.DepthBias					= 0.0f;
		TraceContext.SampleCount				= RenderVolumetricCloudParameters.CloudSkyAOSampleCount;
		TraceContext.SizeInvSize				= RenderVolumetricCloudParameters.CloudSkyAOSizeInvSize;
		TraceContext.TracingSizeInvSize			= TraceContext.SizeInvSize;
		TraceContext.TracingPixelScaleOffset    = float4(1.0f, 1.0f, 0.0f, 0.0f);
		TraceContext.TranslatedWorldToLightClipMatrix		= RenderVolumetricCloudParameters.CloudSkyAOTranslatedWorldToLightClipMatrix;
		TraceContext.TranslatedWorldToLightClipMatrixInv	= RenderVolumetricCloudParameters.CloudSkyAOTranslatedWorldToLightClipMatrixInv;
		TraceContext.TraceDir					= RenderVolumetricCloudParameters.CloudSkyAOTraceDir;
	}

	FMaterialPixelParameters MaterialParameters = MakeInitializedMaterialPixelParameters();
	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, SvPosition, true);


	FCloudLayerParameters CloudLayerParams = GetCloudLayerParams(
		RenderVolumetricCloudParameters.CloudLayerCenterKm, RenderVolumetricCloudParameters.PlanetRadiusKm,
		RenderVolumetricCloudParameters.BottomRadiusKm, RenderVolumetricCloudParameters.TopRadiusKm);
	float2 UV = float2(SvPosition.xy * TraceContext.TracingPixelScaleOffset.xy + TraceContext.TracingPixelScaleOffset.zw) * TraceContext.SizeInvSize.zw;

	const float NearZDepth = 1.0f; // using FReversedZOrthoMatrix
	float3 NearClipPlaneWorldPos = CloudShadowUvToWorldSpace(NearZDepth, UV, TraceContext.TranslatedWorldToLightClipMatrixInv) - DFHackToFloat(PrimaryView.PreViewTranslation); // TODO_ keep world pos as LWCVEctor
	const float3 LightDirection = TraceContext.TraceDir; // It points from light to surface


	// Compute the min and max distance to trace (in the cloud layer)
	float TMin = -999999999.0f;
	float TMax = -999999999.0f;
	float3 RayOrigin = NearClipPlaneWorldPos;
	float3 RayOriginKm = RayOrigin * CENTIMETER_TO_KILOMETER;
	float2 tTop2 = 0.0f;
	bool bTraceTop = false;
	if (RayIntersectSphereSolution(RayOriginKm, LightDirection, float4(RenderVolumetricCloudParameters.CloudLayerCenterKm, RenderVolumetricCloudParameters.TopRadiusKm), tTop2))
	{
		bTraceTop = true;
		float2 tBottom2 = 0.0f;
		if (RayIntersectSphereSolution(RayOriginKm, LightDirection, float4(RenderVolumetricCloudParameters.CloudLayerCenterKm, RenderVolumetricCloudParameters.BottomRadiusKm), tBottom2))
		{
			// If we see both intersection in front of us, keep the min/closest, otherwise the max/furthest
			float TempTop = all(tTop2 > 0.0f) ? min(tTop2.x, tTop2.y) : max(tTop2.x, tTop2.y);
			float TempBottom = all(tBottom2 > 0.0f) ? min(tBottom2.x, tBottom2.y) : max(tBottom2.x, tBottom2.y);

			if (all(tBottom2 > 0.0f))
			{
				// But if we can see the bottom of the layer, make sure we use the camera or the highest top layer intersection
				TempTop = max(0.0f, min(tTop2.x, tTop2.y));
			}
			else
			{
				// We are inside under the cloud layer, we simply skip shadowing evaluation and mark near clip as front depth (remove >50% of the cost at dusk/dawn time)
				OutColor0 = float3(0.0f, 0.0f, 0.0f);
				return;
			}

			TMin = min(TempBottom, TempTop);
			TMax = max(TempBottom, TempTop);
		}
		else
		{
			// Only intersecting with the top atmosphere, we have our min and max t
			TMin = tTop2.x;
			TMax = tTop2.y;
		}
	}
	else
	{
		// No intersection with the top of the cloud layer
		OutColor0 = float3(TraceContext.FarDepthKm, 0.0f, 0.0f);
		return;
	}

	TMin = max(0.0f, TMin) * KILOMETER_TO_CENTIMETER;
	TMax = max(0.0f, TMax) * KILOMETER_TO_CENTIMETER;
	float ClosestIntersection = TMin; // Stay on the near clip plane if we are under the top layer.
	float3 WorldPosOnLayer = NearClipPlaneWorldPos + LightDirection * ClosestIntersection;



	float3 ExtinctionAcc = 0.0f;
	float ExtinctionAccCount = 0.0f;
	float3 MaxOpticalDepth = 0.0f;
	const float DefaultFarDepth = TraceContext.FarDepthKm * KILOMETER_TO_CENTIMETER;
	float NearDepth = DefaultFarDepth;

	const float LayerHeight = CloudLayerParams.TopRadius - CloudLayerParams.BottomRadius;
	const float ShadowLengthTest = TMax - TMin;
	const float ShadowStepCount = TraceContext.SampleCount;
	const float InvShadowStepCount = 1.0f / ShadowStepCount;
	const float ShadowDtMeter = ShadowLengthTest * CENTIMETER_TO_METER / ShadowStepCount;
	if (bTraceTop)
	{
		// Linear shadow samples (reference)
		for (float ShadowT = 0.5; ShadowT < ShadowStepCount; ShadowT += 1.0f)
		{
			const float SampleT = ShadowLengthTest * (ShadowT * InvShadowStepCount);
			// WorldPositionDDX/Y only need to be set when using analytical derivatives, which is not needed in pixel shaders
			UpdateMaterialCloudParam(MaterialParameters, (WorldPosOnLayer + LightDirection * SampleT) + DFHackToFloat(PrimaryView.PreViewTranslation), 0.0f /*WorldPositionDDX*/, 0.0f /*WorldPositionDDY*/,			// LWC TODO FIX ME
				ResolvedView, CloudLayerParams, SampleT, SPACE_SKIPPING_SLICE_DEPTH_OFF);

		#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY
			if (MaterialParameters.VolumeSampleConservativeDensity.x <= 0.0f)
			{
				continue; // Conservative density is 0 so skip and go to the next sample
			}
		#endif

			VolumetricCloudCalcPixelMaterialInputs(MaterialParameters, PixelMaterialInputs);
			ConvertCloudPixelMaterialInputsToWorkingColorSpace(PixelMaterialInputs);
			float3 ShadowExtinctionCoefficients = SampleExtinctionCoefficients(PixelMaterialInputs);

			bool MediumPresent = any(ShadowExtinctionCoefficients > 0.0f);
			NearDepth = MediumPresent ? min(NearDepth, SampleT) : NearDepth;

			ExtinctionAcc += ShadowExtinctionCoefficients;
			MaxOpticalDepth += ShadowExtinctionCoefficients * ShadowDtMeter;
			ExtinctionAccCount += MediumPresent ? 1.0f : 0.0f;
		}
	}

	// We output front depth and also the mean path extinction that is going to be scaled later by the real path length behind the front depth.
	// We also scale extinction and optical depth by the global cloud shadow strength.
	const float MeanGreyExtinction = TraceContext.Strength * dot(ExtinctionAcc / max(1.0f, ExtinctionAccCount), float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
	const float MaxGreyOpticalDepth = TraceContext.Strength * dot(MaxOpticalDepth, float3(1.0f / 3.0f, 1.0f / 3.0f, 1.0f / 3.0f));
	const bool NoHit = NearDepth == DefaultFarDepth;
	const float FrontDepth = NoHit ? TMax * CENTIMETER_TO_KILOMETER : (ClosestIntersection + NearDepth) * CENTIMETER_TO_KILOMETER;
	OutColor0 = float3(max(0.0f, FrontDepth + TraceContext.DepthBias), MeanGreyExtinction , MaxGreyOpticalDepth);
}

#endif // SHADER_SHADOW_PS



#if SHADER_SHADOW_FILTER_CS

#include "ShaderPrint.ush"
#include "MonteCarlo.ush"

SamplerState BilinearSampler;
Texture2D<float3> CloudShadowTexture;
RWTexture2D<float3> OutCloudShadowTexture;
float4 CloudTextureSizeInvSize;
float4 CloudTextureTexelWorldSizeInvSize;
float CloudLayerStartHeight;
float CloudSkyAOApertureScaleMul;
float CloudSkyAOApertureScaleAdd;

struct CloudShadowData
{
	float DepthKm;
	float MeanExtinction;
	float MaxOpticalDepth;
};

CloudShadowData CloudShadowData_LoadSrc(uint2 Coord)
{
	float3 TexData = CloudShadowTexture.Load(uint3(Coord, 0));
	CloudShadowData CloudData;
	CloudData.DepthKm = TexData.x;
	CloudData.MeanExtinction = TexData.y;
	CloudData.MaxOpticalDepth = TexData.z;
	return CloudData;
}

CloudShadowData CloudShadowData_Sample(float2 Coord)
{
	float3 TexData = CloudShadowTexture.SampleLevel(BilinearSampler, Coord, 0);
	CloudShadowData CloudData;
	CloudData.DepthKm = TexData.x;
	CloudData.MeanExtinction = TexData.y;
	CloudData.MaxOpticalDepth = TexData.z;
	return CloudData;
}

void CloudShadowData_WriteDst(in CloudShadowData CloudData, uint2 Coord)
{
	OutCloudShadowTexture[Coord] = float3(CloudData.DepthKm, CloudData.MeanExtinction, CloudData.MaxOpticalDepth);
}

[numthreads(8, 8, 1)]
void MainShadowFilterCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();


	if (all(DispatchThreadId.xy < uint2(CloudTextureSizeInvSize.xy)))
	{
		const int2 CenterCoord = int2(DispatchThreadId.xy);
		const float2 CenterUV  = (float2(DispatchThreadId.xy) + 0.5f) * CloudTextureSizeInvSize.zw;

#if PERMUTATION_SKYAO

		CloudShadowData Center = CloudShadowData_LoadSrc(CenterCoord);

		float DepthKm = 0.0f;
		float MeanExtinction = 0.0f;
		float MaxOpticalDepth = 0.0f;
		float SampleCount = 0.0f;

		// SkyAO filter is not a 2d bluer of the volumetric cloud top/bottom shadow map but an integration over the hemisphere at the ground level.
		// It does not take into account the planet curvature.
		// We filter transmittance to control the max extinction. Because visibility is the important value to integrate over the hemisphere (not extinction).

		float MaxTransmittance = 0.0f;

		// Hardcoded 64 samples over the hemisphere
		float SampleMaxU = 4.0f;
		float SampleIncU = 1.0f / SampleMaxU;
		float SampleMaxV = 4.0f;
		float SampleIncV = 1.0f / SampleMaxV;
		//UNROLL
		for (float U = 0.5 * SampleIncU; U < SampleMaxU; U += 1.0f)
		{
			for (float V = 0.5 * SampleIncV; V < SampleMaxV; V += 1.0f)
			{
				const float ZetaU = U / SampleMaxU;
				const float ZetaV = CloudSkyAOApertureScaleAdd + CloudSkyAOApertureScaleMul * (V / SampleMaxV); // User sample scale to only sample a solid angle smaller than the hemisphere.
				float4 SampleDir  = UniformSampleHemisphere(float2(U / SampleMaxU, ZetaV));
				float2 WorldOffset = SampleDir.xy * CloudLayerStartHeight / max(0.0000001f, SampleDir.z);
				float2 TexelOffset = WorldOffset * CloudTextureTexelWorldSizeInvSize.zw;
				float2 UVOffset    = TexelOffset * CloudTextureSizeInvSize.zw;

				CloudShadowData Data = CloudShadowData_Sample(CenterUV + UVOffset);	// TODO generate and fetch different mip based on area from projected solid angle.
				DepthKm += Data.DepthKm;
				MeanExtinction += Data.MeanExtinction;
				MaxOpticalDepth += Data.MaxOpticalDepth;
				MaxTransmittance += exp(-Data.MaxOpticalDepth);
			}
		}
		SampleCount = SampleMaxU * SampleMaxV;

		// For the output, we average all the linear quantities over the solid angle.
		CloudShadowData OutputCloudData;
		OutputCloudData.DepthKm = DepthKm / SampleCount;
		OutputCloudData.MeanExtinction	= MeanExtinction / SampleCount;
		OutputCloudData.MaxOpticalDepth	= MaxTransmittance > 0.0f ? -log(MaxTransmittance / SampleCount) : 100.0f;

		CloudShadowData_WriteDst(OutputCloudData, CenterCoord);

#else

		CloudShadowData Data0 = CloudShadowData_LoadSrc(CenterCoord * 2 + int2(0, 0));
		CloudShadowData Data1 = CloudShadowData_LoadSrc(CenterCoord * 2 + int2(1, 0));
		CloudShadowData Data2 = CloudShadowData_LoadSrc(CenterCoord * 2 + int2(1, 1));
		CloudShadowData Data3 = CloudShadowData_LoadSrc(CenterCoord * 2 + int2(0, 1));
		
		float Mean				= Data0.DepthKm + Data1.DepthKm + Data2.DepthKm + Data3.DepthKm;
		float StandardDeviation = (abs(Data0.DepthKm - Mean) + abs(Data1.DepthKm - Mean) + abs(Data2.DepthKm - Mean) + abs(Data3.DepthKm - Mean)) * 0.25;
		Data0.DepthKm			= Mean - StandardDeviation; // Filtered front depth instead of only mean

		Data0.MeanExtinction	+= Data1.MeanExtinction + Data2.MeanExtinction + Data3.MeanExtinction;
		Data0.MaxOpticalDepth	+= Data1.MaxOpticalDepth + Data2.MaxOpticalDepth + Data3.MaxOpticalDepth;

		Data0.MeanExtinction	*= 1.0f / 4.0f;
		Data0.MaxOpticalDepth	*= 1.0f / 4.0f;

		CloudShadowData_WriteDst(Data0, CenterCoord);

#endif
	}


}

#endif // SHADER_SHADOW_FILTER_CS



#if SHADER_DEBUG_SHADOW_CS

#include "ShaderPrint.ush"

Texture2D<float2> CloudTracedTexture;
float4 CloudTextureSizeInvSize;
float3 CloudTraceDirection;
float4x4 CloudTranslatedWorldToLightClipMatrixInv;

[numthreads(8, 8, 1)]
void MainDrawDebugShadowCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	//ResolvedView = ResolveView();

	if (all(DispatchThreadId.xy < uint2(CloudTextureSizeInvSize.xy)))
	{
		float2 CloudShadowData = CloudTracedTexture.Load(uint3(DispatchThreadId.xy, 0));
		float NearDepth = CloudShadowData.x * KILOMETER_TO_CENTIMETER;
		float OpticalDepth = CloudShadowData.y;

		const float NearZDepth = 1.0f; // using FReversedZOrthoMatrix
		float3 NearClipPlaneWorldPos00 = CloudShadowUvToWorldSpace(NearZDepth, float2(DispatchThreadId.xy+uint2(0,0)) * CloudTextureSizeInvSize.zw, CloudTranslatedWorldToLightClipMatrixInv);
		float3 NearClipPlaneWorldPos01 = CloudShadowUvToWorldSpace(NearZDepth, float2(DispatchThreadId.xy+uint2(0,1)) * CloudTextureSizeInvSize.zw, CloudTranslatedWorldToLightClipMatrixInv);
		float3 NearClipPlaneWorldPos11 = CloudShadowUvToWorldSpace(NearZDepth, float2(DispatchThreadId.xy+uint2(1,1)) * CloudTextureSizeInvSize.zw, CloudTranslatedWorldToLightClipMatrixInv);
		float3 NearClipPlaneWorldPos10 = CloudShadowUvToWorldSpace(NearZDepth, float2(DispatchThreadId.xy+uint2(1,0)) * CloudTextureSizeInvSize.zw, CloudTranslatedWorldToLightClipMatrixInv);

		//

		NearClipPlaneWorldPos00 += CloudTraceDirection * NearDepth;
		NearClipPlaneWorldPos01 += CloudTraceDirection * NearDepth;
		NearClipPlaneWorldPos11 += CloudTraceDirection * NearDepth;
		NearClipPlaneWorldPos10 += CloudTraceDirection * NearDepth;

		float4 DebugColor = OpticalDepth > 0.0f ? float4(float2(DispatchThreadId.xy) * CloudTextureSizeInvSize.zw, 0.1f, 1.0f)
			: float4(0.0f, 0.0f, 0.0f, 0.20f);

		if (OpticalDepth > 0.0f)
		{
			AddQuadWS(NearClipPlaneWorldPos00, NearClipPlaneWorldPos01, NearClipPlaneWorldPos11, NearClipPlaneWorldPos10, DebugColor);
		}
	}
}

#endif // SHADER_DEBUG_SHADOW_CS



#if SHADER_SHADOW_TEMPORAL_PROCESS_CS

SamplerState BilinearSampler;
Texture2D<float3> CurrCloudShadowTexture;
Texture2D<float3> PrevCloudShadowTexture;
RWTexture2D<float3> OutCloudShadowTexture;

float4x4 CurrFrameCloudShadowmapTranslatedWorldToLightClipMatrixInv;
float4x4 PrevFrameCloudShadowmapTranslatedWorldToLightClipMatrix;

float3 CurrFrameLightPos;
float3 PrevFrameLightPos;
float3 CurrFrameLightDir;
float3 PrevFrameLightDir;
uint CloudShadowMapAnchorPointMoved;

float4 CloudTextureSizeInvSize;
float4 CloudTextureTracingSizeInvSize;
float4 CloudTextureTracingPixelScaleOffset;
float TemporalFactor;
uint PreviousDataIsValid;

[numthreads(8, 8, 1)]
void MainShadowTemporalProcessCS(uint3 DispatchThreadId : SV_DispatchThreadID)
{
	ResolvedView = ResolveView();

	if (all(DispatchThreadId.xy < uint2(CloudTextureSizeInvSize.xy)))
	{
		const int2 CenterCoord = int2(DispatchThreadId.xy);
		const float2 CenterUV = (float2(DispatchThreadId.xy) + 0.5f) * CloudTextureSizeInvSize.zw;

		// Source coord from half resolution
		const int2 NewTracedDataCoord = CenterCoord / int2(CloudTextureTracingPixelScaleOffset.xy);
		// Destination coord for full resolution
		const int2 NewDataCoord = int2(NewTracedDataCoord * int2(CloudTextureTracingPixelScaleOffset.xy) + int2(CloudTextureTracingPixelScaleOffset.zw));
		// Sampel data for this frame
		float3 CurrCloudShadowData = CurrCloudShadowTexture.Load(uint3(NewTracedDataCoord, 0));

		float3 FilteredData = CurrCloudShadowData;

		if (PreviousDataIsValid)
		{
			const float DummyDepth = 0.0f;
			float4 ClipCoord = float4(CenterUV * float2(2.0f, -2.0f) + float2(-1.0f, 1.0f), DummyDepth, 1.0f); 
			float4 HomogeneousCoord = mul(ClipCoord, CurrFrameCloudShadowmapTranslatedWorldToLightClipMatrixInv);
			float3 WorldPos = HomogeneousCoord.xyz / HomogeneousCoord.www;
			ClipCoord = mul(float4(WorldPos, 1.0f), PrevFrameCloudShadowmapTranslatedWorldToLightClipMatrix);
			ClipCoord /= ClipCoord.wwww;
			float2 PreviousUVs = 0.5f + float2(0.5f, -0.5f) * ClipCoord.xy;

			// We have a new shadow data to blending with history
			float3 PrevCloudShadowData = PrevCloudShadowTexture.SampleLevel(BilinearSampler, PreviousUVs, 0);

			if (CloudShadowMapAnchorPointMoved > 0) 
			{
				// Approximated reprojection of previous depth into current orthographic volume, to make sure nothing pops when the orthographic projection is translated around on the planet
				// Only when the anchor poitn as moved, to have light rotation still smoothed out
				float3 PrevPos = PrevFrameLightPos + PrevFrameLightDir * PrevCloudShadowData.r * KILOMETER_TO_CENTIMETER;
				PrevCloudShadowData.r = dot(PrevPos - CurrFrameLightPos, CurrFrameLightDir) * CENTIMETER_TO_KILOMETER;
			}

			if (all(NewDataCoord == CenterCoord))
			{
				// A very simple filter for now that does work well for reasonable atmospheric light rotations.
				FilteredData = PrevCloudShadowData + TemporalFactor * (CurrCloudShadowData - PrevCloudShadowData);
				FilteredData.x = CurrCloudShadowData.x; // Do not filter depth otherwise we have trouble converging due to precision issue. This will need to be fixed
			}
			else
			{
				// Simple reproject history
				FilteredData = PrevCloudShadowData;
			}
		}

		OutCloudShadowTexture[CenterCoord] = max(0.0, FilteredData);
	}
}

#endif // SHADER_SHADOW_FILTER_CS


