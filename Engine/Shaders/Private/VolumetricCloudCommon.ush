// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * VolumetricCloudCommon.usf: Common functions for passes using volumetric cloud data.
 */



#ifndef D_VOLUMETRIC_CLOUD_COMMON
#define D_VOLUMETRIC_CLOUD_COMMON


#include "Common.ush"

#if USE_CLOUD_TRANSMITTANCE
	Texture2D<float3> CloudShadowmapTexture;
	SamplerState CloudShadowmapSampler;
	float CloudShadowmapFarDepthKm;
	float4x4 CloudShadowmapTranslatedWorldToLightClipMatrix;
	float CloudShadowmapStrength;
#endif

// Uv in [0,1]
// ZDepth = 1.0 for near clip plane (See FReversedZOrthoMatrix)
// returns Position returns in centimeters
float3 CloudShadowUvToWorldSpace(in float ZDepth, in float2 UV, in float4x4 WorldToLightClipShadowMatrixInv)
{
	const float NearZDepth = 1.0f; // using FReversedZOrthoMatrix
	float4 ClipCoord = float4(UV * float2(2.0f, -2.0f) + float2(-1.0f, 1.0f), ZDepth, 1.0f);
	float4 HomogeneousCoord = mul(ClipCoord, WorldToLightClipShadowMatrixInv);
	float3 WorldPos = HomogeneousCoord.xyz / HomogeneousCoord.www;
	return WorldPos;
}



// WorldSpace in centimeters
// returns UV in [0,1]
// returns UV in [0,1]
float2 CloudShadowWorldSpaceToUv(in float3 TranslatedWorldPosition, in float4x4 TranslatedWorldToLightClipMatrix, inout float z)
{
	float4 ClipSpace = mul(float4(TranslatedWorldPosition, 1.0f), TranslatedWorldToLightClipMatrix);
	ClipSpace /= ClipSpace.wwww;
	z = ClipSpace.z;
	float2 UVs = ClipSpace.xy;
	UVs = 0.5f + float2(0.5f, -0.5f) * UVs;
	return UVs;
}



// WorldSpace in centimeters
// returns transmittance
float GetCloudVolumetricShadow(in float3 TranslatedWorldPosition, in float4x4 CloudShadowmapTranslatedWorldToLightClipMatrix, in float ShadowmapFarDepthKm,
	Texture2D<float3> VolumetricCloudShadowMapTexture, SamplerState VolumetricCloudShadowMapTextureSampler, inout float OutOpticalDepth)
{
	float CloudShadowSampleZ = 0.0f;
	float2 UVs = CloudShadowWorldSpaceToUv(TranslatedWorldPosition, CloudShadowmapTranslatedWorldToLightClipMatrix, CloudShadowSampleZ);
	const float3 CloudShadowData = VolumetricCloudShadowMapTexture.SampleLevel(VolumetricCloudShadowMapTextureSampler, UVs.xy, 0).rgb;
	const float ShadowFrontDepthKm = CloudShadowData.r;	// unit kilometer
	const float MeanExtinction = CloudShadowData.g;		// unit 1/m
	const float MaxOpticalDepth = CloudShadowData.b;	// unitless
	const float SampleDepthKm = saturate(1.0f - CloudShadowSampleZ) * ShadowmapFarDepthKm;
	OutOpticalDepth = MeanExtinction * (max(0.0f, SampleDepthKm - ShadowFrontDepthKm) * KILOMETER_TO_METER);
	OutOpticalDepth = min(MaxOpticalDepth, OutOpticalDepth);
	const float CloudShadowFactor = saturate(exp(-OutOpticalDepth));
	return CloudShadowFactor;
}







float4 GetCloudLuminanceTransmittanceOverFog(
	float4 NDC, float3 SampledWorldPos, float3 CameraWorldPos,
	Texture2D<float4> VolumetricCloudColor, SamplerState VolumetricCloudColorSampler,
	Texture2D<float4> VolumetricCloudDepth, SamplerState VolumetricCloudDepthSampler,
	float OneOverPreExposure, float4 CurrentVertexFog, float SoftBlendingDistanceKm, 
	float2 UVScale, float2 UVMax)
{
	float2 ScreenUv = (NDC.xy / NDC.ww) * float2(0.5f, -0.5f) + 0.5f;
	ScreenUv *= UVScale;
	ScreenUv = min(ScreenUv, UVMax);


#if FEATURE_LEVEL >= FEATURE_LEVEL_SM5
	// When depth is reprojected using bilinear filtering, it can leave some trails that are visible when doing binary depth testing.
	// Fixing the reprojection is costly and also not perfect so instead we workaround that issue but using the min of the gathered cloud front depth (being MaxHalf when no clouds have been traced)
	float4 FrontDepth4 = VolumetricCloudDepth.Gather(VolumetricCloudDepthSampler, ScreenUv, int2(0, 0));
	const float CloudFrontDepthKm = min(min(FrontDepth4.x, FrontDepth4.y), min(FrontDepth4.z, FrontDepth4.w));
#else
	const float CloudFrontDepthKm = Texture2DSample(VolumetricCloudDepth, VolumetricCloudDepthSampler, ScreenUv).r;
#endif

	const float DistanceFromCameraKm = length(SampledWorldPos.xyz - CameraWorldPos) * CENTIMETER_TO_KILOMETER;
	const float DepthInsideKm = DistanceFromCameraKm - CloudFrontDepthKm;
	float4 FinalFog = CurrentVertexFog;
	if (DepthInsideKm > 0.0f)
	{
		float4 CloudFog = Texture2DSample(VolumetricCloudColor, VolumetricCloudColorSampler, ScreenUv);
		CloudFog.rgb *= OneOverPreExposure;

		const float BlendFactor = saturate(DepthInsideKm / SoftBlendingDistanceKm);
		const float4 FogToApplyUnder = CurrentVertexFog;
		const float4 FogToApplyOver = lerp(float4(0.0, 0.0, 0.0, 1.0), CloudFog, BlendFactor);

		// Apply any other fog under the cloud contribution because the cloud data alreayd has heightfog and aerial perspective applied on it.
		// It is an approximation but it works well practically.
		FinalFog.rgb = FogToApplyOver.rgb + FogToApplyUnder.rgb * FogToApplyOver.a;
		// And combine both transmittance.
		FinalFog.a = FogToApplyOver.a * FogToApplyUnder.a;
	}

	return FinalFog;
}







#endif // D_VOLUMETRIC_CLOUD_COMMON


