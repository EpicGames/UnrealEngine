// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * VolumetricCloudMaterialPixelCommon.usf: Common functions for passes using volumetric cloud data in relation to material pixel parameters.
 */



#ifndef D_VOLUMETRIC_CLOUD_MATERIAL_PIXEL_COMMON
#define D_VOLUMETRIC_CLOUD_MATERIAL_PIXEL_COMMON

#include "Common.ush"

struct FCloudLayerParameters
{
	float3 CloudLayerCenter;
	float PlanetRadius;
	float BottomRadius;
	float TopRadius;
	float ToNormAltitude;
};

FCloudLayerParameters GetCloudLayerParams(
	in float3 CloudLayerCenterKm,
	in float PlanetRadiusKm,
	in float BottomRadiusKm,
	in float TopRadiusKm)
{
	FCloudLayerParameters CloudLayerParams;
	CloudLayerParams.CloudLayerCenter = CloudLayerCenterKm * KILOMETER_TO_CENTIMETER;
	CloudLayerParams.PlanetRadius = PlanetRadiusKm * KILOMETER_TO_CENTIMETER;
	CloudLayerParams.BottomRadius = BottomRadiusKm * KILOMETER_TO_CENTIMETER;
	CloudLayerParams.TopRadius = TopRadiusKm * KILOMETER_TO_CENTIMETER;
	CloudLayerParams.ToNormAltitude = 1.0 / (CloudLayerParams.TopRadius - CloudLayerParams.BottomRadius);
	return CloudLayerParams;
}

#if CLOUD_LAYER_PIXEL_SHADER

#define TRACING_SHADOW_DISTANCE_OFF	0.0f

#define SPACE_SKIPPING_SLICE_DEPTH_OFF 0.0f

// Function to update the material parameter structure for an evaluation
void UpdateMaterialCloudParam(inout FMaterialPixelParameters MaterialParameters,
                              float3 TranslatedWorldPosition, float3 WorldPositionDDX, float3 WorldPositionDDY,
							  in ViewState InputView, in FCloudLayerParameters CloudLayerParams,
                              in float ShadowSampleDistance, in float EmptySpaceSkippingSliceDepth)
{
	MaterialParameters.AbsoluteWorldPosition = DFFastSubtract(TranslatedWorldPosition, InputView.PreViewTranslation);
	MaterialParameters.LWCData.AbsoluteWorldPosition = LWCSubtract(TranslatedWorldPosition, MaterialParameters.LWCData.PreViewTranslation);
	MaterialParameters.WorldPosition_CamRelative = TranslatedWorldPosition;
	//MaterialParameters.WorldPosition_NoOffsets;				// TODO
	//MaterialParameters.LWCData.WorldPosition_NoOffsets;	// TODO
	//MaterialParameters.WorldPosition_NoOffsets_CamRelative;	// TODO
	MaterialParameters.CameraVector = select(IsOrthoProjection(InputView), InputView.ViewForward, normalize(InputView.TranslatedWorldCameraOrigin - TranslatedWorldPosition));
	MaterialParameters.WorldPosition_DDX = WorldPositionDDX;
	MaterialParameters.WorldPosition_DDY = WorldPositionDDY;

	float3 AbsoluteWorldPositionTruncated = TranslatedWorldPosition - DFHackToFloat(InputView.PreViewTranslation);
	float CloudSampleAltitude = length(AbsoluteWorldPositionTruncated - CloudLayerParams.CloudLayerCenter);
	MaterialParameters.CloudSampleAltitude = CloudSampleAltitude - CloudLayerParams.PlanetRadius;
	MaterialParameters.CloudSampleAltitudeInLayer = CloudSampleAltitude - CloudLayerParams.BottomRadius;
	MaterialParameters.CloudSampleNormAltitudeInLayer = saturate(MaterialParameters.CloudSampleAltitudeInLayer * CloudLayerParams.ToNormAltitude);

	const float DefaultConservativeDensity = 1.0f;
	MaterialParameters.VolumeSampleConservativeDensity = DefaultConservativeDensity;								// Defaults to "medium is potentially present" in case it is not fed by the user.
#if MATERIAL_VOLUMETRIC_ADVANCED_CONSERVATIVE_DENSITY
	MaterialParameters.VolumeSampleConservativeDensity = GetVolumetricAdvancedMaterialOutput6(MaterialParameters);	// Evaluate conservative density
#endif

	MaterialParameters.CloudEmptySpaceSkippingSphereCenterWorldPosition = AbsoluteWorldPositionTruncated; // This is the same as the WorldPosition node but float3
	MaterialParameters.CloudEmptySpaceSkippingSphereRadius = 0.5f * EmptySpaceSkippingSliceDepth * 1.73f;			// 1.73 = sqrt(1*1+1*1+1*1), the lenght of the diagonal of a cube as compared to its unit edge length

	MaterialParameters.ShadowSampleDistance = ShadowSampleDistance;
}

#endif // CLOUD_LAYER_PIXEL_SHADER


#endif // D_VOLUMETRIC_CLOUD_MATERIAL_PIXEL_COMMON


