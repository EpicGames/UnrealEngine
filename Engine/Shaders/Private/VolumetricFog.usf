// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricFog.usf 
=============================================================================*/

#include "Common.ush"
#include "Definitions.usf"
#define SUPPORT_CONTACT_SHADOWS 0
#define USE_SOURCE_TEXTURE USE_RECT_LIGHT_TEXTURE
#include "DeferredLightingCommon.ush"
#include "LightGridCommon.ush"
#include "HeightFogCommon.ush"
#include "SHCommon.ush"
#if DISTANCE_FIELD_SKY_OCCLUSION
#include "DistanceFieldAOShared.ush"
#include "DistanceField/GlobalDistanceFieldShared.ush"
#endif
#include "VolumeLightingCommon.ush"
#include "VolumetricLightmapShared.ush"
#include "ReflectionEnvironmentShared.ush"
#include "ForwardShadowingCommon.ush"
#include "ParticipatingMediaCommon.ush"
#include "LightDataUniforms.ush"
#if LUMEN_GI
#include "Lumen/LumenTranslucencyVolumeShared.ush"
#endif
#include "Random.ush"

#if VIRTUAL_SHADOW_MAP
#include "VirtualShadowMaps/VirtualShadowMapProjectionCommon.ush"
#endif

#if USE_LOCAL_FOG_VOLUMES
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif

#ifndef USE_EMISSIVE
#define USE_EMISSIVE 1
#endif

#ifndef USE_RAYTRACED_SHADOWS
#define USE_RAYTRACED_SHADOWS 0
#endif

RWTexture3D<float4> RWVBufferA;
#if USE_EMISSIVE
RWTexture3D<float4> RWVBufferB;
#endif

#if !SHADING_PATH_MOBILE
#define USE_CLOUD_TRANSMITTANCE 1
#include "VolumetricCloudCommon.ush"
#endif

float ComputeDepthFromZSlice(float ZSlice)
{
	return ComputeDepthFromZSlice(VolumetricFog.GridZParams, ZSlice);
}

float ComputeZSliceFromDepth(float SliceDepth)
{
	return ComputeZSliceFromDepth(VolumetricFog.GridZParams, SliceDepth);
}

float4x4 UnjitteredClipToTranslatedWorld;    
float4x4 UnjitteredPrevTranslatedWorldToClip;

float3 ComputeCellTranslatedWorldPosition(uint3 GridCoordinate, float3 CellOffset, out float SceneDepth)
{
	float2 VolumeUV = (GridCoordinate.xy + CellOffset.xy) / VolumetricFog.ViewGridSize.xy;
	float2 VolumeNDC = (VolumeUV * 2 - 1) * float2(1, -1);

	SceneDepth = ComputeDepthFromZSlice(max(GridCoordinate.z + CellOffset.z, 0.0f));

	float TileDeviceZ = ConvertToDeviceZ(SceneDepth);
	float4 CenterPosition = mul(float4(VolumeNDC, TileDeviceZ, 1), UnjitteredClipToTranslatedWorld);
	return CenterPosition.xyz / CenterPosition.w;
}

float3 ComputeCellTranslatedWorldPosition(uint3 GridCoordinate, float3 CellOffset)
{
	float Unused;
	return ComputeCellTranslatedWorldPosition(GridCoordinate, CellOffset, Unused);
}

float GridCenterOffsetFromDepthBuffer;
float GridCenterOffsetThresholdToAcceptDepthBufferOffset;

// Applying an offset from the depth buffer helps with reducing lighting on fog leaking through walls
void ApplyDepthConstraintsToOffset(float3 GridCoordinate, inout float3 LocalFrameJitterOffset)
{
	if (GridCenterOffsetFromDepthBuffer >= 0.0)
	{
		float Unused;
		float4 GridSampleClip = mul(float4(ComputeCellTranslatedWorldPosition(GridCoordinate, LocalFrameJitterOffset, Unused), 1), PrimaryView.TranslatedWorldToClip);
		float3 GridSampleScreen = GridSampleClip.xyz / GridSampleClip.w;
		float3 GridSampleUVz = float3(GridSampleScreen.xy * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz, GridSampleScreen.z);
		float GridSampleSceneDepth = CalcSceneDepth(uint2(GridSampleUVz.xy * View.BufferSizeAndInvSize.xy));

		float DepthGridCoordinateWithZOffset = ComputeZSliceFromDepth(GridSampleSceneDepth) - GridCenterOffsetFromDepthBuffer;

		float3 GridCoordinateOffset = float3(GridCoordinate) + LocalFrameJitterOffset;
		
		const float DepthToSampleGridOffset = DepthGridCoordinateWithZOffset - GridCoordinateOffset.z; // <0 if samples is in front of the depth buffer
		if (DepthToSampleGridOffset < 0.0 // Move the sample forward only if the depth buffer grid coordinate is closer,
			&& (-DepthToSampleGridOffset < GridCenterOffsetThresholdToAcceptDepthBufferOffset) // But only if it is less than N slice away (to not bring all the sample behind forward)
			)
		{
			// If the grid sample is further than the grid z for the depth buffer, adjust the offset 
			LocalFrameJitterOffset.z += DepthToSampleGridOffset;
		}
	}
}

float3 RaleighScattering()
{
	float3 Wavelengths = float3(650.0f, 510.0f, 475.0f);
	float ParticleDiameter = 60;
	float ParticleRefractiveIndex = 1.3f;

	float3 ScaleDependentPortion = pow(ParticleDiameter, 6) / pow(Wavelengths, 4);
	float RefractiveIndexPortion = (ParticleRefractiveIndex * ParticleRefractiveIndex - 1) / (ParticleRefractiveIndex * ParticleRefractiveIndex + 2);
	return (2 * pow(PI, 5) * RefractiveIndexPortion * RefractiveIndexPortion) * ScaleDependentPortion / 3.0f;
}

float3 ScatteringFunction()
{
	return 1;
	//return RaleighScattering();
}

float4 GlobalAlbedo;
float4 GlobalEmissive;
float GlobalExtinctionScale;

#ifndef THREADGROUP_SIZE 
#define THREADGROUP_SIZE 1
#endif

#ifndef THREADGROUP_SIZE_X
#define THREADGROUP_SIZE_X THREADGROUP_SIZE
#endif

#ifndef THREADGROUP_SIZE_Y
#define THREADGROUP_SIZE_Y THREADGROUP_SIZE
#endif

#ifndef THREADGROUP_SIZE_Z
#define THREADGROUP_SIZE_Z THREADGROUP_SIZE
#endif

#ifdef MaterialSetupCS
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, THREADGROUP_SIZE)]
void MaterialSetupCS( 
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint3 GridCoordinate = DispatchThreadId;

	// Center of the voxel
	float3 VoxelOffset = .5f;

	ApplyDepthConstraintsToOffset(GridCoordinate, VoxelOffset);
	 
	float3 WorldPosition = ComputeCellTranslatedWorldPosition(GridCoordinate, VoxelOffset) - DFHackToFloat(PrimaryView.PreViewTranslation);

	float GlobalDensityFirst = FogStruct.ExponentialFogParameters3.x * exp2(-FogStruct.ExponentialFogParameters.y * (WorldPosition.z - FogStruct.ExponentialFogParameters3.y));
	float GlobalDensitySecond = FogStruct.ExponentialFogParameters2.z * exp2(-FogStruct.ExponentialFogParameters2.y * (WorldPosition.z - FogStruct.ExponentialFogParameters2.w));
	float GlobalDensity = GlobalDensityFirst + GlobalDensitySecond;

	float3 Albedo = GlobalAlbedo.rgb;

	// Exponential height fog interprets density differently, match its behavior
	float MatchHeightFogFactor = .5f;
	GlobalDensity *= MatchHeightFogFactor;
	float Extinction = max(GlobalDensity * GlobalExtinctionScale, 0);

	float3 Scattering = Albedo * Extinction;

	float3 AdditionalEmissive = 0.0f;

#if USE_LOCAL_FOG_VOLUMES
	float3 CellOffset = .5f;

	ApplyDepthConstraintsToOffset(GridCoordinate, CellOffset);

	float SceneDepth;
	const float3 TranslatedWorldPosition			= ComputeCellTranslatedWorldPosition(GridCoordinate, CellOffset, SceneDepth);
	const float3 BackCellTranslatedWorldPosition	= ComputeCellTranslatedWorldPosition(GridCoordinate, float3(CellOffset.x, CellOffset.y, 1.0f), SceneDepth);
	const float  DistanceToCellBack					= GetDistanceToCameraFromViewVector(BackCellTranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);
	if(DistanceToCellBack > LFVGlobalStartDistance)
	{
		const float3 FrontCellTranslatedWorldPosition	= ComputeCellTranslatedWorldPosition(GridCoordinate, float3(CellOffset.x, CellOffset.y, 0.0f), SceneDepth);
		const float DistanceToCellFront					= GetDistanceToCameraFromViewVector(FrontCellTranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);

		float4 VertexClipSpacePosition = mul(float4(TranslatedWorldPosition.xyz, 1), PrimaryView.TranslatedWorldToClip);
		float2 SvPosition = (VertexClipSpacePosition.xy / VertexClipSpacePosition.w * float2(.5f, -.5f) + .5f) * PrimaryView.ViewSizeAndInvSize.xy;
		uint2 TilePos = clamp(uint2(SvPosition.xy / float(LFVTilePixelSize)), uint2(0, 0), LFVTileDataResolution - 1);

		uint LFVCount = LFVTileDataTex[uint3(TilePos, 0)];

		for (uint LFVIndex = 0; LFVIndex < LFVCount; ++LFVIndex)
		{
			uint InstanceIndex = LFVTileDataTex[uint3(TilePos, 1 + LFVIndex)];

			// LFV_TODO skip if out of sphere of influence
			FLocalFogVolumeGPUInstanceData FogInstance = GetLocalFogVolumeGPUInstanceData(InstanceIndex);

			float SoftDensityScale = 1;
			const float LFVStartDistance = max(LFVGlobalStartDistance, FogInstance.StartDistance);

			if (DistanceToCellBack < LFVStartDistance)
			{
				continue;
			}

			if (DistanceToCellFront < LFVStartDistance && LFVStartDistance < DistanceToCellBack)
			{
				// Smoothly transition from the front clip planes of LFV to avoid bad straircase/aliasing visual effect.
				// Extinction is basically scaled as a fonction of the start distance position inside the froxel along V.
				SoftDensityScale = (DistanceToCellBack - LFVStartDistance) / max(0.0001, DistanceToCellBack - DistanceToCellFront);
			}

			const float FogInstanceSoftDensityScale = SoftDensityScale * (DistanceToCellBack - FogInstance.StartDistance) / max(0.0001, DistanceToCellBack - DistanceToCellFront);

			float3 UnitSpacePos = mul(TranslatedWorldPosition - FogInstance.TranslatedWorlPos, FogInstance.PreTranslatedInvTransform);

			float DistToCenter = length(UnitSpacePos - float3(0.0f, 0.0f, 0.0f));
			float SphereFade = saturate(1.0f - DistToCenter);

			// Evaluate both radial and height local fog contribution
			float LFVExtinctionHeight = SphereFade <= 0.0f ? 0.0f : FogInstance.HeightFogExtinction * exp(-FogInstance.HeightFogFalloff * (UnitSpacePos.z - FogInstance.HeightFogOffset));
			float LFVExtinctionRadial = SphereFade <= 0.0f ? 0.0f : FogInstance.RadialFogExtinction * pow(SphereFade, 0.82); // 0.82 is a correction factor needed to get a better match. It is unclear why it is needed...

			// As for the local fog volume analitical evaluation, we want to combine both two optical depth for the "opacity"=1-transmittance to be combined together.
			// This helps to have soft edges when using height fog. 
			/*
			In this case, we have to find math so that the combined extinction resulting in a transmittance representing the multiplication of coverage = 1 - transmittance factors.

			Reminder
				Transmittance as T
				Optical Depth as Tau
				T   =  exp(-Tau)
				Tau = -log(T)		(*)

			We want to combine the both mode transmittance Tr (radial) and Th (height) as done in LocalFogVolumeEvaluateAnalyticalIntegral to have
				 T		= 1 - (1-Tr)(1-Th)											// Where (1-Tr) is virtual coverage of the medium expressed from transmittance for the radial fog.
				 T		= 1 - (1 - exp(-Tau_r)) (1 - exp(-Tau_h)))					// Where Tr = exp(-Tau_r)
				-Tau	= log(1 - (1 - exp(-Tau_r)) (1 - exp(-Tau_h)))				// Using (*)
				 Tau	=-log(exp(-Tau_r) - exp((-Tau_r-Tau_h) + exp(-Tau_h)))

			Optical Depth Tau = Sigma_t * dt so it is linear with respect to dt and extinction Sigma_t. So we can also write
				Sigma_T = -log(exp(-Sigma_T_r) - exp((-Sigma_T_r-Sigma_T_h) + exp(-Sigma_T_h)))

			Factoring some values a bit we can get
				A		= exp(-Sigma_T_r)
				B		= exp(-Sigma_T_h)
				Sigma_T	= -log(A - A*B + B)
			*/

			// Optical Depth is linear with Extinction so there is no need to account for the step length.
			float FactorA = exp(-LFVExtinctionHeight);
			float FactorB = exp(-LFVExtinctionRadial);
			float LFVExtinction = -log(FactorA - FactorA * FactorB + FactorB);
			LFVExtinction = max(0.0f, LFVExtinction);

			// Convert extinction from unit volume 1/m into extinction for volumetric fog as 1/cm
			LFVExtinction *= (1.0f / METER_TO_CENTIMETER);

			LFVExtinction = min(LFVMaxDensityIntoVolumetricFog, LFVExtinction) * SoftDensityScale;

			float3 LFVScattering = LFVExtinction * FogInstance.Albedo;

			Scattering += LFVScattering;
			Extinction += LFVExtinction;

	#if USE_EMISSIVE
			// Emissive follow the density of the medium
			AdditionalEmissive += LFVExtinction * FogInstance.Emissive;
	#endif
		}
	}
#endif // USE_LOCAL_FOG_VOLUMES

	if (all((int3)GridCoordinate < VolumetricFog.ResourceGridSizeInt))
	{
		RWVBufferA[GridCoordinate] = float4(Scattering, Extinction);
#if USE_EMISSIVE
		float3 HeightFogEmissive = GlobalEmissive.rgb;
	#if PROJECT_EXPFOG_MATCHES_VFOG
		HeightFogEmissive.rgb *= GlobalDensity; // Emissive needs to follow the height fog density for a better match
	#endif 
		RWVBufferB[GridCoordinate] = float4(HeightFogEmissive.rgb + AdditionalEmissive, 0);
#endif
	}
}
#endif

// Positive g = forward scattering
// Zero g = isotropic
// Negative g = backward scattering
float PhaseFunction(float g, float CosTheta)
{
	return HenyeyGreensteinPhase(g, CosTheta);
}

struct FWriteToSliceVertexOutput
{
	FScreenVertexOutput Vertex;
#if USING_VERTEX_SHADER_LAYER
	uint LayerIndex : SV_RenderTargetArrayIndex;
#else
	uint LayerIndex : TEXCOORD1;
#endif
};

/** Z index of the minimum slice in the range. */
int MinZ;
float4 ViewSpaceBoundingSphere;
float4x4 ViewToVolumeClip;

/** Vertex shader that writes to a range of slices of a volume texture. */
void WriteToBoundingSphereVS(
	float2 InPosition : ATTRIBUTE0,
	float2 InUV       : ATTRIBUTE1,
	uint LayerIndex : SV_InstanceID,
	out FWriteToSliceVertexOutput Output
	)
{
	float SliceDepth = ComputeDepthFromZSlice(LayerIndex + MinZ);
	float SliceDepthOffset = abs(SliceDepth - ViewSpaceBoundingSphere.z);

	if (SliceDepthOffset < ViewSpaceBoundingSphere.w)
	{
		// Compute the radius of the circle formed by the intersection of the bounding sphere and the current depth slice
		float SliceRadius = sqrt(ViewSpaceBoundingSphere.w * ViewSpaceBoundingSphere.w - SliceDepthOffset * SliceDepthOffset);
		// Place the quad vertex to tightly bound the circle
		float3 ViewSpaceVertexPosition = float3(ViewSpaceBoundingSphere.xy + (InUV * 2 - 1) * SliceRadius, SliceDepth);
		Output.Vertex.Position = mul(float4(ViewSpaceVertexPosition, 1), ViewToVolumeClip);
		float2 ClipRatio = float2(1.0f, 1.0f);

		// Scale rendered position to account for frustum difference between fog voxelization and rendered scene
		Output.Vertex.Position.xy *= ClipRatio;
	}
	else
	{
		// Slice does not intersect bounding sphere, emit degenerate triangle
		Output.Vertex.Position = 0;
	}

	// Debug - draw to entire texture in xy
	//Output.Vertex.Position = float4(InUV * float2(2, -2) + float2(-1, 1), 0, 1);

	Output.Vertex.UV = 0;
	Output.LayerIndex = LayerIndex + MinZ;
}

float HistoryWeight;
float LightScatteringSampleJitterMultiplier;
float4 FrameJitterOffsets[16]; 
uint HistoryMissSuperSampleCount;
float PhaseG;
float InverseSquaredLightDistanceBiasScale;
int VirtualShadowMapId;

#ifndef USE_SUPER_SAMPLING
#define USE_SUPER_SAMPLING 0
#endif

#include "LightFunctionAtlas/LightFunctionAtlasCommon.usf"

Texture2D<float> ConservativeDepthTexture;
float2 PrevConservativeDepthTextureSize;
Texture2D<float> PrevConservativeDepthTexture;
uint UseConservativeDepthTexture;

#if USE_RAYTRACED_SHADOWS

#include "/Engine/Private/RayTracing/RayTracingCommon.ush"

RaytracingAccelerationStructure TLAS;

float ComputeRayTracedShadowFactor(float3 TranslatedWorldPosition, float3 LightDirection, float TMax)
{
	uint RayFlags = RAY_FLAG_SKIP_CLOSEST_HIT_SHADER | RAY_FLAG_ACCEPT_FIRST_HIT_AND_END_SEARCH;
	uint RaytracingMask = RAY_TRACING_MASK_SHADOW | RAY_TRACING_MASK_THIN_SHADOW;

	FRayDesc Ray;
	Ray.Origin = TranslatedWorldPosition;
	Ray.Direction = LightDirection;
	Ray.TMin = 0.0f;
	Ray.TMax = TMax;

	FMinimalPayload MinimalPayload = TraceVisibilityRay(
		TLAS,
		RayFlags,
		RaytracingMask,
		Ray);

	if (MinimalPayload.IsHit())
	{
		return 0.0f;
	}

	return 1.0;
}

#endif

uint bUseTemporalReprojection;
uint bDistanceFieldSkyOcclusion;
uint bLumenGI;
uint bVirtualShadowMap;
uint bUseRaytracedShadowsVolume;
uint bUseLightFunctionAtlas;
uint bUseMegaLights;
uint bUseLightSoftFading;
uint bUseSuperSampling;

float4 InjectShadowedLocalLightCommon(uint3 GridCoordinate)
{
	float4 OutScattering = 0;

	// Somehow pixels are being rasterized outside of the viewport on a 970 GTX, perhaps due to use of a geometry shader bypassing the viewport scissor.
	// This triggers the HistoryMissSuperSampleCount path causing significant overhead for shading off-screen pixels.
	if (all(int3(GridCoordinate) < VolumetricFog.ResourceGridSizeInt))
	{
		FDeferredLightData LightData = InitDeferredLightFromUniforms();

		float VolumetricScatteringIntensity = DeferredLightUniforms.VolumetricScatteringIntensity;

		float3 L = 0;
		float3 ToLight = 0;

		uint NumSuperSamples = 1;

		#if USE_TEMPORAL_REPROJECTION
		IF_UBERSHADER(bUseTemporalReprojection)
		{
			float3 HistoryUV = ComputeHistoryVolumeUVFromTranslatedPos(ComputeCellTranslatedWorldPosition(GridCoordinate, .5f), UnjitteredPrevTranslatedWorldToClip);
			float HistoryAlpha = HistoryWeight;

			FLATTEN
			if (any(HistoryUV < 0) || any(HistoryUV > 1))
			{
				HistoryAlpha = 0;
			}

			NumSuperSamples = HistoryAlpha < .001f ? HistoryMissSuperSampleCount : 1;
		}
		#endif

		const bool bSoftFadeEnabled = 
			#if UBERSHADER
			bUseLightSoftFading &&
			#endif
			VolumetricFog.LightSoftFading > 0;

		for (uint SampleIndex = 0; SampleIndex < NumSuperSamples; SampleIndex++)
		{
			float3 CellOffset = FrameJitterOffsets[SampleIndex].xyz;
			//float3 CellOffset = .5f;

			ApplyDepthConstraintsToOffset(GridCoordinate, CellOffset);

			float SceneDepth;
			float3 TranslatedWorldPosition = ComputeCellTranslatedWorldPosition(GridCoordinate, CellOffset, SceneDepth);
			float3 CameraVector = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);

			float CellRadius = length(TranslatedWorldPosition - ComputeCellTranslatedWorldPosition(GridCoordinate + uint3(1, 1, 1), CellOffset));
			float Cell2DRadius = length(TranslatedWorldPosition - ComputeCellTranslatedWorldPosition(GridCoordinate + uint3(1, 1, 0), CellOffset));
			float LightVolumetricSoftFadeDistance = VolumetricFog.LightSoftFading * Cell2DRadius;
			// Bias the inverse squared light falloff based on voxel size to prevent aliasing near the light source
			float DistanceBias = max(CellRadius * InverseSquaredLightDistanceBiasScale, 1);

			float3 LightColor = DeferredLightUniforms.Color;
			float LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);

			float Lighting;
			if( LightData.bRectLight )
			{
				FRect Rect = GetRect(ToLight, LightData);

				float SoftFade = 1.0f;
#if USE_LIGHT_SOFT_FADING
				if (bSoftFadeEnabled)
				{
					SoftFade = GetRectLightVolumetricSoftFading(LightData, Rect, LightVolumetricSoftFadeDistance, ToLight);
				}
#endif // USE_LIGHT_SOFT_FADING

				Lighting = SoftFade * IntegrateLight(Rect);

#if USE_RECT_LIGHT_TEXTURE
				if (IsRectVisible(Rect))
				{
					const FRectTexture SourceTexture = ConvertToRectTexture(LightData);
					const float3 FalloffColor = SampleSourceTexture(Lighting, Rect, SourceTexture);
					LightColor *= FalloffColor;
				}
#endif // USE_RECT_LIGHT_TEXTURE
			}
			else
			{
				FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
				Capsule.DistBiasSqr = Pow2(DistanceBias);

				float SoftFade = 1.0f;
#if USE_LIGHT_SOFT_FADING
				if (LightData.bSpotLight && bSoftFadeEnabled)
				{
					SoftFade = GetSpotLightVolumetricSoftFading(LightData, LightVolumetricSoftFadeDistance, ToLight);
				}
#endif // USE_LIGHT_SOFT_FADING

				Lighting = SoftFade * IntegrateLight(Capsule, LightData.bInverseSquared);
			}

			float CombinedAttenuation = Lighting * LightMask;
			float ShadowFactor = 1.0f;

			bool IsPointLight = LightData.bRadialLight && !LightData.bSpotLight;
	
#if ENABLE_SHADOW_COMPUTATION
			if (CombinedAttenuation > 0)
			{
#if USE_RAYTRACED_SHADOWS
				ShadowFactor = ComputeRayTracedShadowFactor(TranslatedWorldPosition, normalize(ToLight), length(ToLight));
#else
				bool bUnused = false;
				ShadowFactor = ComputeVolumeShadowing(TranslatedWorldPosition, IsPointLight, LightData.bSpotLight, bUnused);

				#if VIRTUAL_SHADOW_MAP
				IF_UBERSHADER(bVirtualShadowMap)
				{
					FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapTranslatedWorld(VirtualShadowMapId, TranslatedWorldPosition);
					ShadowFactor *= VirtualShadowMapSample.ShadowFactor;
				}
				#endif // VIRTUAL_SHADOW_MAP

#endif // USE_RAYTRACED_SHADOWS
			}
#endif
	
			FLightFunctionColor LightFunctionColor = 1.0f;
			#if	USE_LIGHT_FUNCTION_ATLAS
			IF_UBERSHADER(bUseLightFunctionAtlas)
			{
				LightFunctionColor = GetLocalLightFunctionCommon(TranslatedWorldPosition, LightData.LightFunctionAtlasLightIndex);
			}
			#endif

			OutScattering.rgb += LightColor * LightFunctionColor * (PhaseFunction(PhaseG, dot(L, -CameraVector)) * CombinedAttenuation * ShadowFactor * VolumetricScatteringIntensity);
				
			// To debug culling
			//OutScattering.rgb += DeferredLightUniforms.Color * .0000001f;
		}

		// We pre-expose the buffer containing shadowed local lights luminance contribution. This helps maintaining details and color at high exposure for very bright scenes.
		OutScattering.rgb *= View.PreExposure / (float)NumSuperSamples;
	}

	return OutScattering;
}

#if USE_RAYTRACED_SHADOWS

RWTexture3D<float4> OutVolumeTexture;
uint3 OutputOffset;
int FirstSlice;

int3 GetVoxelIndex(int ThreadIndex, int3 Resolution)
{
	int SliceSize = Resolution.x * Resolution.y;
	int SliceIndex = ThreadIndex / SliceSize;
	int SliceRemainder = ThreadIndex - SliceIndex * SliceSize;
	
	return int3(
		SliceRemainder % Resolution.x,
		SliceRemainder / Resolution.x,
		SliceIndex
	);
}

RAY_TRACING_ENTRY_RAYGEN(InjectShadowedLocalLightRGS)
{
	int3 Position = GetVoxelIndex(DispatchRaysIndex().x, VolumetricFog.ResourceGridSizeInt);
	Position.z += FirstSlice;

	float4 OutScattering = InjectShadowedLocalLightCommon(Position);

	float4 DestValue = OutVolumeTexture[Position];
	OutVolumeTexture[Position] = DestValue + OutScattering;
}

RWTexture3D<float> OutShadowVolumeTexture;

RAY_TRACING_ENTRY_RAYGEN(InjectShadowedDirectionalLightRGS)
{
	int3 Position = GetVoxelIndex(DispatchRaysIndex().x, VolumetricFog.ResourceGridSizeInt);

	const int SampleIndex = 0;
	uint3 Rand32Bits = Rand4DPCG32(int4(Position.xyz, View.StateFrameIndexMod8 + 8 * SampleIndex)).xyz;
	float3 Rand3D = (float3(Rand32Bits) / float(uint(0xffffffff))) * 2.0f - 1.0f;
	float3 CellOffset = FrameJitterOffsets[SampleIndex].xyz + LightScatteringSampleJitterMultiplier * Rand3D;

	ApplyDepthConstraintsToOffset(Position, CellOffset);

	const float3 TranslatedWorldPosition = ComputeCellTranslatedWorldPosition(Position, CellOffset);
	OutShadowVolumeTexture[Position] = ComputeRayTracedShadowFactor(TranslatedWorldPosition, ForwardLightStruct.DirectionalLightDirection, 1.0e27);
}

#endif // USE_RAYTRACED_SHADOWS

void InjectShadowedLocalLightPS(
	FWriteToSliceGeometryOutput Input,
	out float4 OutScattering : SV_Target0
	)
{
	float4 SvPosition = Input.Vertex.Position;
	OutScattering = 0;

	if (UseConservativeDepthTexture > 0)
	{
		const float SceneDepth = ConservativeDepthTexture.Load(uint3(SvPosition.xy, 0));
		if (SceneDepth > SvPosition.z)
		{
			clip(-1); // If the froxel is behind conservative front depth, skip the light injection.
			return;
		}
	}

	OutScattering = InjectShadowedLocalLightCommon(uint3(SvPosition.xy, Input.LayerIndex));
}

Texture3D<float4> VBufferA;
Texture3D<float4> VBufferB;
uint UseEmissive;

Texture3D<float4> LightScatteringHistory;
SamplerState LightScatteringHistorySampler;
float2 LightScatteringHistoryPreExposureAndInv;

Texture3D<float4> LocalShadowedLightScattering;

RWTexture3D<float4> RWLightScattering;

#if DISTANCE_FIELD_SKY_OCCLUSION

float HemisphereConeTraceAgainstGlobalDistanceFieldClipmap(
	uniform uint ClipmapIndex,
	float3 TranslatedWorldShadingPosition,
	float3 ConeDirection,
	float TanConeHalfAngle)
{
	float MinStepSize = GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].w * 2 / 100.0f;
	float InvAOGlobalMaxOcclusionDistance = 1.0f / AOGlobalMaxOcclusionDistance;

	float MinVisibility = 1;
	float WorldStepOffset = 2;

	LOOP
	for (uint StepIndex = 0; StepIndex < NUM_CONE_STEPS && WorldStepOffset < AOGlobalMaxOcclusionDistance; StepIndex++)
	{
		float3 TranslatedWorldSamplePosition = TranslatedWorldShadingPosition + ConeDirection * WorldStepOffset;
		float DistanceToOccluder = SampleGlobalDistanceField(TranslatedWorldSamplePosition, AOGlobalMaxOcclusionDistance, ClipmapIndex);
		float SphereRadius = WorldStepOffset * TanConeHalfAngle;
		float InvSphereRadius = rcpFast(SphereRadius);

		// Derive visibility from 1d intersection
		float Visibility = saturate(DistanceToOccluder * InvSphereRadius);
			
		float OccluderDistanceFraction = (WorldStepOffset + DistanceToOccluder) * InvAOGlobalMaxOcclusionDistance;

		// Fade out occlusion based on distance to occluder to avoid a discontinuity at the max AO distance
		Visibility = max(Visibility, saturate(OccluderDistanceFraction * OccluderDistanceFraction * .6f));
			
		MinVisibility = min(MinVisibility, Visibility);

		WorldStepOffset += max(DistanceToOccluder, MinStepSize);
	}

	return MinVisibility;
}

float HemisphereConeTraceAgainstGlobalDistanceField(float3 TranslatedWorldShadingPosition, float3 ConeDirection, float TanConeHalfAngle)
{
	int MinClipmapIndex = -1;
	for (uint ClipmapIndex = 0; ClipmapIndex < NumGlobalSDFClipmaps; ++ClipmapIndex)
	{
		float DistanceFromClipmap = ComputeDistanceFromBoxToPointInside(GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].xyz, GlobalVolumeTranslatedCenterAndExtent[ClipmapIndex].www, TranslatedWorldShadingPosition);
		if (DistanceFromClipmap > AOGlobalMaxOcclusionDistance)
		{
			MinClipmapIndex = ClipmapIndex;
			break;
		}
	}

	float MinVisibility = 1.0f;
	if (MinClipmapIndex >= 0)
	{
		MinVisibility = HemisphereConeTraceAgainstGlobalDistanceFieldClipmap(MinClipmapIndex, TranslatedWorldShadingPosition, ConeDirection, TanConeHalfAngle);
	}

	return MinVisibility;
}

#endif

float SkyLightUseStaticShadowing;

float ComputeSkyVisibility(float3 TranslatedWorldPosition, float3 BrickTextureUVs)
{
	float Visibility = 1;

	#if DISTANCE_FIELD_SKY_OCCLUSION
	IF_UBERSHADER(bDistanceFieldSkyOcclusion)
	{
		// Trace one 45 degree cone straight up for sky occlusion
		float TanConeHalfAngle = tan((float)PI / 4);
		Visibility = HemisphereConeTraceAgainstGlobalDistanceField(TranslatedWorldPosition, float3(0, 0, 1), TanConeHalfAngle);
	}
	#endif

#if ALLOW_STATIC_LIGHTING
	if (SkyLightUseStaticShadowing > 0)
	{
		float3 SkyBentNormal = GetVolumetricLightmapSkyBentNormal(BrickTextureUVs);
		Visibility = length(SkyBentNormal);
	}
#endif

	return Visibility;
}

float4x4 DirectionalLightFunctionTranslatedWorldToShadow;
Texture2D DirectionalLightLightFunctionTexture;
SamplerState DirectionalLightLightFunctionSampler;
uint DirectionalApplyLightFunctionFromAtlas;
uint DirectionalLightFunctionAtlasLightIndex;

FLightFunctionColor GetDirectionalLightFunction(float3 TranslatedWorldPosition) // directional light
{
	#if USE_LIGHT_FUNCTION_ATLAS  // Used when injecting lights from the light grid
	IF_UBERSHADER(bUseLightFunctionAtlas)
	{
		if (DirectionalApplyLightFunctionFromAtlas > 0)
		{
			return GetLocalLightFunctionCommon(TranslatedWorldPosition, DirectionalLightFunctionAtlasLightIndex);
		}
	}
	#endif

	float4 HomogeneousShadowPosition = mul(float4(TranslatedWorldPosition, 1), DirectionalLightFunctionTranslatedWorldToShadow);
	float2 LightFunctionUV = HomogeneousShadowPosition.xy * .5f + .5f;
	LightFunctionUV.y = 1 - LightFunctionUV.y;

	return Texture2DSampleLevel(DirectionalLightLightFunctionTexture, DirectionalLightLightFunctionSampler, LightFunctionUV, 0).x;
}

uint SampleSkyLightDiffuseEnvMap;
float2 UseHeightFogColors;	// x=override directional light using height fog inscattering color, y=override sky light using heigh fog inscattering cubemap
float UseDirectionalLightShadowing;
float StaticLightingScatteringIntensity;

#if SHADING_PATH_MOBILE
uint   MobileHasDirectionalLight;
float3 MobileDirectionalLightColor;
float3 MobileDirectionalLightDirection;
#endif

Texture3D<float> RaytracedShadowsVolume;
Texture3D<float3> MegaLightsVolume;

// Modify UV used to sample LightScatteringHistory to avoid cells that were occluded last frame
float2 FixupHistoryUV(float2 UV, float2 Size, float PrevCellNDCPositionZ, out bool bHasValidHistory)
{
	float2 FullResUV = UV * Size;

	uint2 ScreenCoord = floor(FullResUV - 0.5f);
	float2 BilinearWeights = frac(FullResUV - 0.5f);
	float2 GatherUV = (ScreenCoord + 1.0f) / Size;

	float2 FullResOffset = FullResUV - ScreenCoord;

	const float2 PrevConservativeDepthGatherUV = GatherUV / (PrimaryView.VolumetricFogViewGridUVToPrevViewRectUV.xy * PrimaryView.VolumetricFogPrevViewGridRectUVToResourceUV.xy);
	const float4 PrevSceneDepths = PrevConservativeDepthTexture.Gather(LightScatteringHistorySampler, PrevConservativeDepthGatherUV);
	const bool4 ValidSamples = PrevSceneDepths < PrevCellNDCPositionZ;
	
	bHasValidHistory = true;

	if (all(ValidSamples))
	{
		return UV;
	}
	
	// Gather pattern:
    // wz
    // xy
	
	if (all(ValidSamples.wz))
	{
		return (ScreenCoord + float2(FullResOffset.x, 0.5f)) / Size;
	}
	if (all(ValidSamples.xy))
	{
		return (ScreenCoord + float2(FullResOffset.x, 1.5f)) / Size;
	}
	if (all(ValidSamples.wx))
	{
		return (ScreenCoord + float2(0.5f, FullResOffset.y)) / Size;
	}
	if (all(ValidSamples.zy))
	{
		return (ScreenCoord + float2(1.5f, FullResOffset.y)) / Size;
	}

	if (ValidSamples.x)
	{
		return (ScreenCoord + float2(0.5f, 1.5f)) / Size;
	}

	if (ValidSamples.y)
	{
		return (ScreenCoord + float2(1.5f, 1.5f)) / Size;
	}

	if (ValidSamples.w)
	{
		return (ScreenCoord + float2(0.5f, 0.5f)) / Size;
	}

	if (ValidSamples.z)
	{
		return (ScreenCoord + float2(1.5f, 0.5f)) / Size;
	}

	bHasValidHistory = false;
	return UV;
}

#ifdef LightScatteringCS
[numthreads(THREADGROUP_SIZE_X, THREADGROUP_SIZE_Y, THREADGROUP_SIZE_Z)]
void LightScatteringCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint3 GridCoordinate = DispatchThreadId;
	float3 LightScattering = 0;
	uint NumSuperSamples = 1;

#if USE_TEMPORAL_REPROJECTION
	float3 HistoryUV = ComputeHistoryVolumeUVFromTranslatedPos(ComputeCellTranslatedWorldPosition(GridCoordinate, .5f), UnjitteredPrevTranslatedWorldToClip);
	bool bHasValidHistory = true;
#endif

	// If the froxel is behind front depth, do not evaluate any lighting.
	if (UseConservativeDepthTexture > 0)
	{
		// expand by half voxel towards the camera to support bilinear filtering
		const float3 FarDepthOffset = float3(0.5f, 0.5f, -0.5f);
		float3 CellTranslatedWorldPosition = ComputeCellTranslatedWorldPosition(GridCoordinate, FarDepthOffset);

		float4 CellNDCPosition = mul(float4(CellTranslatedWorldPosition, 1), PrimaryView.TranslatedWorldToClip);
		CellNDCPosition.xyz /= CellNDCPosition.w;
		float2 UVs = CellNDCPosition.xy * float2(0.5f, -0.5f) + 0.5f;

		const float SceneDepth = ConservativeDepthTexture.Load(uint3(GridCoordinate.xy, 0));

		if (SceneDepth > CellNDCPosition.z)
		{
			RWLightScattering[GridCoordinate] = float4(0.0f, 0.0f, 0.0f, 0.0f);
			return;
		}

		#if USE_TEMPORAL_REPROJECTION
		IF_UBERSHADER(bUseTemporalReprojection)
		{
			// TODO: investigate whether using CellTranslatedWorldPosition (which is biased towards camera) is correct here
			// it might be more accurate to transform unbiased translated world position to previous frame clip space and apply bias towards camera in that space?
			float4 PrevCellNDCPosition = mul(float4(CellTranslatedWorldPosition, 1), UnjitteredPrevTranslatedWorldToClip);
			PrevCellNDCPosition.xyz /= PrevCellNDCPosition.w;

			HistoryUV.xy = FixupHistoryUV(HistoryUV.xy, View.VolumetricFogPrevResourceGridSize.xy, PrevCellNDCPosition.z, bHasValidHistory);
		}
		#endif
	}

	#if USE_TEMPORAL_REPROJECTION
	float HistoryAlpha = HistoryWeight;
	IF_UBERSHADER(bUseTemporalReprojection)
	{
		// We need to test with View.VolumetricFogPrevUVMaxForTemporalBlend for HistoryUV because that is what we clamp with in ComputeHistoryVolumeUVFromTranslatedPos.
		FLATTEN
		if (any(HistoryUV < 0) || any(HistoryUV >= float3(View.VolumetricFogPrevUVMaxForTemporalBlend, 1.0f)) || !bHasValidHistory)
		{
			HistoryAlpha = 0;
		}

		// Supersample if the history was outside the camera frustum
		// The compute shader is dispatched with extra threads, make sure those don't supersample
		NumSuperSamples = 1;
		#if USE_SUPER_SAMPLING
		IF_UBERSHADER (bUseSuperSampling)
		{
			NumSuperSamples = HistoryAlpha < .001f && all(int3(GridCoordinate) < VolumetricFog.ViewGridSizeInt) ? HistoryMissSuperSampleCount : 1;
		}
		#endif
	}
	#endif

	for (uint SampleIndex = 0; SampleIndex < NumSuperSamples; SampleIndex++)
	{
		uint3 Rand32Bits = Rand4DPCG32(int4(GridCoordinate.xyz, View.StateFrameIndexMod8 + 8 * SampleIndex)).xyz;
		float3 Rand3D = (float3(Rand32Bits) / float(uint(0xffffffff))) * 2.0f - 1.0f;
		float3 CellOffset = FrameJitterOffsets[SampleIndex].xyz + LightScatteringSampleJitterMultiplier * Rand3D;
		//CellOffset = 0.5f;

		ApplyDepthConstraintsToOffset(GridCoordinate, CellOffset);

		float SceneDepth;
		float3 TranslatedWorldPosition = ComputeCellTranslatedWorldPosition(GridCoordinate, CellOffset, SceneDepth);
		float3 WorldPosition = TranslatedWorldPosition - DFHackToFloat(PrimaryView.PreViewTranslation); // LWC_TODO
		float CameraVectorLength = GetDistanceToCameraFromViewVector(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin);
		float3 CameraVector = GetCameraVectorFromTranslatedWorldPosition(TranslatedWorldPosition);
		
	#if SHADING_PATH_MOBILE
		const bool	 bHasDirectionalLight		= MobileHasDirectionalLight > 0;
		const float3 InDirectionalLightColor	= MobileDirectionalLightColor;	// DirectionalLightVolumetricScatteringIntensity is baked in MobileDirectionalLightColor
		const float3 InDirectionalLightDirection= MobileDirectionalLightDirection;
		const bool   bEvaluateDirectionalLight	= true;
	#else
		const bool	 bHasDirectionalLight		= ForwardLightStruct.HasDirectionalLight;
		const float3 InDirectionalLightColor	= ForwardLightStruct.DirectionalLightColor * ForwardLightStruct.DirectionalLightVolumetricScatteringIntensity;
		const float3 InDirectionalLightDirection= ForwardLightStruct.DirectionalLightDirection;
		const bool   bEvaluateDirectionalLight	= !USE_MEGA_LIGHTS || (UBERSHADER && USE_MEGA_LIGHTS && !bUseMegaLights) || !ForwardLightStruct.DirectionalLightHandledByMegaLights;
	#endif
		BRANCH
		if (bHasDirectionalLight && bEvaluateDirectionalLight)
		{
			float ShadowFactor = 1;
			
			if (UseDirectionalLightShadowing > 0)
			{
				ShadowFactor *= ComputeDirectionalLightStaticShadowing(TranslatedWorldPosition);
				bool bUnused = false;
				ShadowFactor *= ComputeDirectionalLightDynamicShadowing(TranslatedWorldPosition, SceneDepth, bUnused);

				#if VIRTUAL_SHADOW_MAP
				IF_UBERSHADER(bVirtualShadowMap)
				{
					FVirtualShadowMapSampleResult VirtualShadowMapSample = SampleVirtualShadowMapTranslatedWorld(ForwardLightStruct.DirectionalLightVSM, TranslatedWorldPosition);
					ShadowFactor *= VirtualShadowMapSample.ShadowFactor;
				}
				#endif

				#if USE_RAYTRACED_SHADOWS_VOLUME
				IF_UBERSHADER(bUseRaytracedShadowsVolume)
				{
					ShadowFactor *= RaytracedShadowsVolume[GridCoordinate];
				}
				#endif

#if !SHADING_PATH_MOBILE
				if (CloudShadowmapStrength > 0.0f)
				{
					float OutOpticalDepth = 0.0f;
					ShadowFactor *= lerp(1.0f, GetCloudVolumetricShadow(TranslatedWorldPosition, CloudShadowmapTranslatedWorldToLightClipMatrix, CloudShadowmapFarDepthKm, CloudShadowmapTexture, CloudShadowmapSampler, OutOpticalDepth), CloudShadowmapStrength);
				}
#endif
			}

			FLightFunctionColor LightFunctionColor = GetDirectionalLightFunction(TranslatedWorldPosition);

			float3 DirectionalLightColor = InDirectionalLightColor;

			if (UseHeightFogColors.x > 0)
			{
				// Attempt to maintain intensity ratio between sky and sun
				DirectionalLightColor = VolumetricFog.HeightFogDirectionalLightInscatteringColor * Luminance(InDirectionalLightColor);
			}

			LightScattering += DirectionalLightColor * LightFunctionColor * ShadowFactor * PhaseFunction(PhaseG, dot(InDirectionalLightDirection, -CameraVector));
		}

		FTwoBandSHVector RotatedHGZonalHarmonic;
		RotatedHGZonalHarmonic.V = float4(1.0f, CameraVector.y, CameraVector.z, CameraVector.x) * float4(1.0f, PhaseG, PhaseG, PhaseG); // Note: I believe PhaseG here hsould be negated to match the sky & environment.

		float3 BrickTextureUVs = 0;

#if ALLOW_STATIC_LIGHTING && !SHADING_PATH_MOBILE // Mobile does not support VLM
		if (View.SkyLightVolumetricScatteringIntensity > 0 || StaticLightingScatteringIntensity > 0)
		{
			BrickTextureUVs = ComputeVolumetricLightmapBrickTextureUVs(WorldPosition);
		}
#endif

		#if LUMEN_GI
		IF_UBERSHADER (bLumenGI)
		{
			// Lumen Dynamic GI + shadowed Skylight
			FTwoBandSHVectorRGB TranslucencyGISH = GetTranslucencyGIVolumeLighting(DFPromote(WorldPosition), PrimaryView.WorldToClip, false); // LUMEN_LWC_TODO

			LightScattering += max(DotSH(TranslucencyGISH, RotatedHGZonalHarmonic), 0);
		}
		#endif

		#if !LUMEN_GI || UBERSHADER
		// Skylight
		IF_UBERSHADER (!bLumenGI)
		{
			if (View.SkyLightVolumetricScatteringIntensity > 0)
			{
				float3 SkyLighting;

				if (UseHeightFogColors.y > 0)
				{
					float3 HeightFogInscatteringColor = ComputeInscatteringColor(CameraVector, CameraVectorLength);
					float ScalarFactor = SHAmbientFunction();
					FTwoBandSHVectorRGB SkyIrradianceSH;
					SkyIrradianceSH.R.V = float4(ScalarFactor * HeightFogInscatteringColor.r, 0, 0, 0);
					SkyIrradianceSH.G.V = float4(ScalarFactor * HeightFogInscatteringColor.g, 0, 0, 0);
					SkyIrradianceSH.B.V = float4(ScalarFactor * HeightFogInscatteringColor.b, 0, 0, 0);

					SkyLighting = max(DotSH(SkyIrradianceSH, RotatedHGZonalHarmonic), 0);
				}
				else
				{
					// NOTE it should be CameraVector * PhaseG. But we keep it like that for backward compatibility for now. (visible when r.Lumen.TranslucencyVolume.Enable 0)
					SkyLighting = SampleSkyLightDiffuseEnvMap > 0 ? View.SkyLightColor.rgb * GetSkySHDiffuseSimple(CameraVector * -PhaseG) : 0.0f.xxx;
				}

				float SkyVisibility = ComputeSkyVisibility(TranslatedWorldPosition, BrickTextureUVs);
				LightScattering += (SkyVisibility * View.SkyLightVolumetricScatteringIntensity) * SkyLighting;
			}
		}
		#endif

#if ALLOW_STATIC_LIGHTING && !SHADING_PATH_MOBILE // Mobile does not support VLM
		// Indirect lighting of Stationary lights and Direct + Indirect lighting of Static lights
		if (StaticLightingScatteringIntensity > 0)
		{
			FTwoBandSHVectorRGB IrradianceSH = GetVolumetricLightmapSH2(BrickTextureUVs);

			LightScattering += (StaticLightingScatteringIntensity / PI) * max(DotSH(IrradianceSH, RotatedHGZonalHarmonic), 0);
		}
#endif

		uint GridIndex = ComputeLightGridCellIndex(GridCoordinate.xy * VolumetricFog.FogGridToPixelXY, SceneDepth, 0);
		const FCulledLightsGridHeader CulledLightsGridHeader = GetCulledLightsGridHeader(GridIndex);

		float CellRadius = length(TranslatedWorldPosition - ComputeCellTranslatedWorldPosition(GridCoordinate + uint3(1, 1, 1), CellOffset));
		float Cell2DRadius = length(TranslatedWorldPosition - ComputeCellTranslatedWorldPosition(GridCoordinate + uint3(1, 1, 0), CellOffset));
		float LightVolumetricSoftFadeDistance = VolumetricFog.LightSoftFading * Cell2DRadius;
		// Bias the inverse squared light falloff based on voxel size to prevent aliasing near the light source
		float DistanceBiasSqr = max(CellRadius * InverseSquaredLightDistanceBiasScale, 1);
		DistanceBiasSqr *= DistanceBiasSqr;

		uint NumLights = CulledLightsGridHeader.NumLights;
		#if USE_MEGA_LIGHTS
		IF_UBERSHADER(bUseMegaLights)
		{
			// when using Mega Lights Volume only need to apply "non-mega" lights here
			NumLights -= CulledLightsGridHeader.NumMegaLights;
		}
		#endif

		// Forward lighting of unshadowed point and spot lights
		LOOP
		for (uint GridLightListIndex = 0; GridLightListIndex < NumLights; GridLightListIndex++)
		{
			const FLocalLightData LocalLight = GetLocalLightDataFromGrid(CulledLightsGridHeader.DataStartIndex + GridLightListIndex, 0);
			#if USE_MEGA_LIGHTS
			IF_UBERSHADER(bUseMegaLights)
			{
				checkSlow(!UnpackIsHandledByMegaLights(LocalLight));
			}
			#endif

			const float VolumetricScatteringIntensity = UnpackVolumetricScatteringIntensity(LocalLight);

			if (VolumetricScatteringIntensity > 0)
			{
				const bool bSoftFadeEnabled = VolumetricFog.LightSoftFading > 0 && bUseLightSoftFading;
				const FDeferredLightData LightData = ConvertToDeferredLight(LocalLight);

				float3 L = 0;
				float3 ToLight = 0;
				float LightMask = GetLocalLightAttenuation(TranslatedWorldPosition, LightData, ToLight, L);

				float Lighting;
				if( LightData.bRectLight )
				{
					FRect Rect = GetRect( ToLight, LightData );

					float SofFade = 1.0f;
#if USE_LIGHT_SOFT_FADING
					if (bSoftFadeEnabled)
					{
						SofFade = GetRectLightVolumetricSoftFading(LightData, Rect, LightVolumetricSoftFadeDistance, ToLight);
					}
#endif // USE_LIGHT_SOFT_FADING

					Lighting = SofFade * IntegrateLight(Rect);
				}
				else
				{
					FCapsuleLight Capsule = GetCapsule(ToLight, LightData);
					Capsule.DistBiasSqr = DistanceBiasSqr;

					float SofFade = 1.0f;
#if USE_LIGHT_SOFT_FADING
					if (LightData.bSpotLight && bSoftFadeEnabled)
					{
						SofFade = GetSpotLightVolumetricSoftFading(LightData, LightVolumetricSoftFadeDistance, ToLight);
					}
#endif // USE_LIGHT_SOFT_FADING

					Lighting = SofFade;
					Lighting *= IntegrateLight(Capsule, LightData.bInverseSquared);
				}

				float CombinedAttenuation = Lighting * LightMask;

				FLightFunctionColor LightFunctionColor = 1.0f;
				#if	USE_LIGHT_FUNCTION_ATLAS
				IF_UBERSHADER(bUseLightFunctionAtlas)
				{

					LightFunctionColor = GetLocalLightFunctionCommon(TranslatedWorldPosition, LightData.LightFunctionAtlasLightIndex);
				}
				#endif

				LightScattering += LightData.Color * LightFunctionColor * (PhaseFunction(PhaseG, dot(L, -CameraVector)) * CombinedAttenuation * VolumetricScatteringIntensity);

				// To debug culling
				//LightScattering += UnpackLightColor(LocalLight) * .0000001f;
			}
		}
	}

	LightScattering /= (float)NumSuperSamples;

	// Shadowed point and spot lights were computed earlier. 
	// Note: this texture was pre-exposed from InjectShadowedLocalLightPS, se we revert pre-exposure.
	LightScattering += View.OneOverPreExposure * LocalShadowedLightScattering[GridCoordinate].xyz;

	#if USE_MEGA_LIGHTS
	IF_UBERSHADER(bUseMegaLights)
	{
		// Inject MegaLights which were computed earlier. This texture was already pre-exposed, so need to rever it here.
		LightScattering += MegaLightsVolume[GridCoordinate] * View.OneOverPreExposure;
	}
	#endif

	float4 MaterialScatteringAndExtinction = VBufferA[GridCoordinate];
	float Extinction = MaterialScatteringAndExtinction.w;
	float3 MaterialEmissive = 0.0f;
	BRANCH
	if (UseEmissive > 0)
	{
		MaterialEmissive = VBufferB[GridCoordinate].xyz;
	}
	float4 PreExposedScatteringAndExtinction = float4(View.PreExposure * (LightScattering * MaterialScatteringAndExtinction.xyz + MaterialEmissive), Extinction);

	#if USE_TEMPORAL_REPROJECTION
	IF_UBERSHADER(bUseTemporalReprojection)
	{
		BRANCH
		if (HistoryAlpha > 0)
		{
			float4 PreExposedHistoryScatteringAndExtinction = Texture3DSampleLevel(LightScatteringHistory, LightScatteringHistorySampler, min(HistoryUV, float3(View.VolumetricFogPrevUVMax, 1.0f)), 0);
			PreExposedHistoryScatteringAndExtinction.rgb *= LightScatteringHistoryPreExposureAndInv.y * View.PreExposure; // Bring previous frame pre exposed history luminance as current frame pre exposed luminance. Leave extinction untouched! 
			PreExposedScatteringAndExtinction = lerp(PreExposedScatteringAndExtinction, PreExposedHistoryScatteringAndExtinction, HistoryAlpha);
		}
	}
	#endif

	// Visualize history rejection for debugging purposes
	#if 0 && USE_TEMPORAL_REPROJECTION
	IF_UBERSHADER(bUseTemporalReprojection)
	{
		if (HistoryAlpha < 0.001f)
		{
			PreExposedScatteringAndExtinction = float4(View.PreExposure * float3(1, 0, 0), Extinction);
		}
	}
	#endif

	if (all(int3(GridCoordinate) < VolumetricFog.ResourceGridSizeInt))
	{
		PreExposedScatteringAndExtinction = MakePositiveFinite(PreExposedScatteringAndExtinction);
		RWLightScattering[GridCoordinate] = PreExposedScatteringAndExtinction;
	}
}
#endif

Texture3D<float4> LightScattering;
RWTexture3D<float4> RWIntegratedLightScattering;

float VolumetricFogNearFadeInDistanceInv;

#ifdef FinalIntegrationCS
[numthreads(THREADGROUP_SIZE, THREADGROUP_SIZE, 1)]
void FinalIntegrationCS(
	uint3 GroupId : SV_GroupID,
	uint3 DispatchThreadId : SV_DispatchThreadID,
	uint3 GroupThreadId : SV_GroupThreadID)
{
	uint3 GridCoordinate = DispatchThreadId;

	float3 AccumulatedLighting = 0;
	float AccumulatedTransmittance = 1.0f;
	float3 PreviousSliceTranslatedWorldPosition = ComputeCellTranslatedWorldPosition(uint3(GridCoordinate.xy, 0), float3(0.5f, 0.5f, 0.0f));
	float AccumulatedDepth = 0.0;
	for (int LayerIndex = 0; LayerIndex < VolumetricFog.ViewGridSizeInt.z; LayerIndex++)
	{
		uint3 LayerCoordinate = uint3(GridCoordinate.xy, LayerIndex);
		float4 PreExposedScatteringAndExtinction = LightScattering[LayerCoordinate];

		float3 LayerTranslatedWorldPosition = ComputeCellTranslatedWorldPosition(LayerCoordinate, .5f);
		float StepLength = length(LayerTranslatedWorldPosition - PreviousSliceTranslatedWorldPosition);
		PreviousSliceTranslatedWorldPosition = LayerTranslatedWorldPosition;

		float Transmittance = exp(-PreExposedScatteringAndExtinction.w * StepLength);

		AccumulatedDepth += StepLength;

		// Fade in as a function of depth
		float FadeInLerpValue = saturate(AccumulatedDepth * VolumetricFogNearFadeInDistanceInv);

		// See "Physically Based and Unified Volumetric Rendering in Frostbite"
		#define ENERGY_CONSERVING_INTEGRATION 1
		#if ENERGY_CONSERVING_INTEGRATION
			float3 ScatteringIntegratedOverSlice = FadeInLerpValue * (PreExposedScatteringAndExtinction.rgb - PreExposedScatteringAndExtinction.rgb * Transmittance) / max(PreExposedScatteringAndExtinction.w, .00001f);
			AccumulatedLighting += ScatteringIntegratedOverSlice * AccumulatedTransmittance;
		#else
			AccumulatedLighting += FadeInLerpValue * PreExposedScatteringAndExtinction.rgb * AccumulatedTransmittance * StepLength;
		#endif
		
		AccumulatedTransmittance *= lerp(1.0f, Transmittance, FadeInLerpValue);

		RWIntegratedLightScattering[LayerCoordinate] = float4(AccumulatedLighting, AccumulatedTransmittance);
	}
}
#endif
