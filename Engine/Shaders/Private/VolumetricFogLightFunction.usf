// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricFogLightFunction.usf
=============================================================================*/

#include "Common.ush"
#include "/Engine/Generated/Material.ush"
#include "LightFunctionCommon.ush"

float4x4 ShadowToTranslatedWorld;

/** Fade distance in x, disabled brightness in y, output for preview shadows mask in z. */
float3 LightFunctionParameters2;
float3 LightTranslatedWorldPosition;
float2 LightFunctionTexelSize;

void Main(
	in noperspective float4 UV : TEXCOORD0,
	in FStereoPSInput StereoInput,
	in float4 SvPosition : SV_Position,
	out float4 OutColor : SV_Target0
	)
{
	ResolvedView = ResolveView();

	float4 TranslatedWorldPosition = float4(0,0,0,0);
	float3 LightVector = float3(0,0,0);

	float Depth = 0.000001f;
	float2 ScreenPosition = 0.0f;

#if LIGHT_TYPE == 0			// Point light
	
	float Theta = (UV.x * (2*PI)) - PI;
	float Phi = UV.y * PI;
	float X = cos(Theta) * sin(Phi);
	float Y = sin(Theta) * sin(Phi);
	float Z = cos(Phi);
	LightVector = float3(Z, Y, X);
	TranslatedWorldPosition = float4(LightTranslatedWorldPosition + LightVector.zyx, 1);

#elif LIGHT_TYPE == 1		// Spot light
	
	ScreenPosition = (SvPosition.xy * LightFunctionTexelSize);
	ScreenPosition.x = 1 - ScreenPosition.x;
	ScreenPosition.y = 1 - ScreenPosition.y;

	// Wrap ScreenPosition to within the atlas tile
	ScreenPosition = frac(ScreenPosition);

	// Compute translated world position (inverse shadow projection) from the screen position
	// make SvPosition appear to be rasterized with the depth from the depth buffer
	TranslatedWorldPosition = mul(float4(ScreenPosition, Depth, 1), ShadowToTranslatedWorld);
	TranslatedWorldPosition.xyz /= TranslatedWorldPosition.w;

	// Compute the light vector containing UV from the translated world position
	float4 Hom = mul(float4(TranslatedWorldPosition.xyz, 1), LightFunctionTranslatedWorldToLight);
	LightVector = Hom.xyz / Hom.w;

#elif LIGHT_TYPE == 2		// Rect light

	ScreenPosition = (SvPosition.xy * LightFunctionTexelSize - float2(0.5f, 0.5f)) * float2(2, 2);
	ScreenPosition.xy = frac(ScreenPosition.xy);

	// Compute translated world position (inverse shadow projection) from the screen position
	// make SvPosition appear to be rasterized with the depth from the depth buffer
	TranslatedWorldPosition = mul(float4(ScreenPosition, Depth, 1), ShadowToTranslatedWorld);
	TranslatedWorldPosition.xyz /= TranslatedWorldPosition.w;

	// Compute the light vector containing UV from the translated world position
	float4 Hom = mul(float4(TranslatedWorldPosition.xyz, 1), LightFunctionTranslatedWorldToLight);
	LightVector = Hom.xyz / Hom.w;

	LightVector.xyz = LightVector.zyx;	// Swap to match UV convention

#else //LIGHT_TYPE == 3		// Directional light

	// Directional light
	ScreenPosition = (SvPosition.xy * LightFunctionTexelSize - float2(0.5f, 0.5f)) * float2(2, -2);

	// make SvPosition appear to be rasterized with the depth from the depth buffer
	TranslatedWorldPosition = mul(float4(ScreenPosition, Depth, 1), ShadowToTranslatedWorld);
	TranslatedWorldPosition.xyz /= TranslatedWorldPosition.w;

	float4 Hom = mul(float4(TranslatedWorldPosition.xyz, 1), LightFunctionTranslatedWorldToLight);
	LightVector = Hom.xyz / Hom.w;

#endif
	
	float3 LightFunction = GetLightFunctionColor(LightVector, TranslatedWorldPosition.xyz);
	float GreyScale = dot(LightFunction, .3333f).x;

	// Calculate radial view distance for stable fading
	float ViewDistance = GetDistanceToCameraFromViewVector(PrimaryView.TranslatedWorldCameraOrigin - TranslatedWorldPosition.xyz);

	float DistanceFadeAlpha = saturate((LightFunctionParameters2.x - ViewDistance) / (LightFunctionParameters2.x * .2f));
	// Fade to disabled based on LightFunctionFadeDistance
	GreyScale = lerp(LightFunctionParameters2.y, GreyScale, DistanceFadeAlpha);

	// Fade to disabled based on ShadowFadeFraction
	GreyScale = lerp(LightFunctionParameters2.y, GreyScale, LightFunctionParameters.y);

	OutColor = GreyScale;
}
