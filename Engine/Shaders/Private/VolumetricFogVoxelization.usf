// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricFogVoxelization.usf
=============================================================================*/

// Change this to force recompilation of all shaders referencing this file.
#pragma message("UESHADERMETADATA_VERSION 43567765-38CF-4EC3-B288-49B6F14EF801")

// So vertex factories can interpolate dependencies of VertexFactoryGetTranslatedPrimitiveVolumeBounds
#define PASS_NEEDS_PRIMITIVE_VOLUME_BOUNDS 1

// Force inclusion of VertexId FVertexFactoryInput even if not otherwise required
#define PASS_NEEDS_VERTEX_ID 1

#include "Common.ush"

#define SceneTexturesStruct VoxelizeVolumePass.SceneTextures 

#include "/Engine/Generated/Material.ush"
#include "/Engine/Generated/VertexFactory.ush"

#include "VolumetricCloudMaterialPixelCommon.ush"

#define VolumetricFog VoxelizeVolumePass.VolumetricFog

uint VoxelizationPassIndex;

float SampleExtinctionCoefficients(in FPixelMaterialInputs PixelMaterialInputs)
{
	float Extinction = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	Extinction = VOLUMETRICFOGCLOUD_EXTINCTION(BSDF).r;
#else
	#if !MATERIAL_SHADINGMODEL_UNLIT
	Extinction = GetMaterialSubsurfaceDataRaw(PixelMaterialInputs).r;
	#endif
#endif
	return clamp(Extinction, 0.0f, 65000.0f);
}

float3 SampleEmissive(in FPixelMaterialInputs PixelMaterialInputs)
{
	float3 EmissiveColor = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	EmissiveColor = BSDF_GETEMISSIVE(BSDF);
#else
	EmissiveColor = GetMaterialEmissiveRaw(PixelMaterialInputs);
#endif
	return clamp(EmissiveColor, 0.0f, 65000.0f);
}

float3 SampleAlbedo(in FPixelMaterialInputs PixelMaterialInputs, in ViewState InputView)
{
	float3 Albedo = 0.0f;
#if TEMPLATE_USES_SUBSTRATE
	FSubstrateBSDF BSDF = PixelMaterialInputs.FrontMaterial.InlinedBSDF;
	Albedo = VOLUMETRICFOGCLOUD_ALBEDO(BSDF);
#else
	#if !MATERIAL_SHADINGMODEL_UNLIT
	Albedo = GetMaterialBaseColor(PixelMaterialInputs);
	#endif
#endif
	return saturate(Albedo)* InputView.DiffuseOverrideParameter.w + InputView.DiffuseOverrideParameter.xyz;
}

struct FVoxelizeVolumePrimitiveVSToGS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float2 VertexQuadCoordinate : ATTRIBUTE0;
};

struct FVoxelizeVolumePrimitiveGSToPS
{
	FVertexFactoryInterpolantsVSToPS FactoryInterpolants;
	float4 OutPosition : SV_POSITION;
	uint SliceIndex : SV_RenderTargetArrayIndex;
};

int ComputeZSliceFromDepth(float SceneDepth)
{
	return (int)(log2(SceneDepth * VolumetricFog.GridZParams.x + VolumetricFog.GridZParams.y) * VolumetricFog.GridZParams.z);
}

float ComputeDepthFromZSlice(float ZSlice)
{
	float SliceDepth = (exp2(ZSlice / VolumetricFog.GridZParams.z) - VolumetricFog.GridZParams.y) / VolumetricFog.GridZParams.x;
	return SliceDepth;
}

#define PRIMITIVE_SPHERE_MODE 0
#define OBJECT_BOX_MODE 1

#ifndef MAX_SLICES_PER_VOXELIZATION_PASS
#define MAX_SLICES_PER_VOXELIZATION_PASS 1 
#endif

#if USING_VERTEX_SHADER_LAYER
#define NUM_INPUT_VERTICES 1
void Voxelize(FVoxelizeVolumePrimitiveVSToGS Inputs[1], out FVoxelizeVolumePrimitiveGSToPS Output)
#else
#define NUM_INPUT_VERTICES 3
[maxvertexcount(MAX_SLICES_PER_VOXELIZATION_PASS * 3)]
void VoxelizeGS(triangle FVoxelizeVolumePrimitiveVSToGS Inputs[3], inout TriangleStream<FVoxelizeVolumePrimitiveGSToPS> OutStream)
#endif
{
#if !USING_VERTEX_SHADER_LAYER
	ResolvedView = ResolveView();
#endif

	int GridSizeZ = VolumetricFog.ViewGridSize.z;
	float4 PrimitiveVolumeBounds = VertexFactoryGetTranslatedPrimitiveVolumeBounds(Inputs[0].FactoryInterpolants);
	float3 ViewSpacePrimitiveVolumeOrigin = mul(float4(PrimitiveVolumeBounds.xyz, 1), ResolvedView.TranslatedWorldToView).xyz;

	uint PrimitiveId = VertexFactoryGetPrimitiveId(Inputs[0].FactoryInterpolants);
	float3 LocalObjectBoundsMin = GetPrimitiveData(PrimitiveId).LocalObjectBoundsMin;
	float3 LocalObjectBoundsMax = GetPrimitiveData(PrimitiveId).LocalObjectBoundsMax;
	FDFMatrix LocalToWorld = GetPrimitiveData(PrimitiveId).LocalToWorld;

	float QuadDepth = ViewSpacePrimitiveVolumeOrigin.z;

	int FurthestSliceIndexUnclamped = ComputeZSliceFromDepth(QuadDepth + PrimitiveVolumeBounds.w);
	int ClosestSliceIndexUnclamped = ComputeZSliceFromDepth(QuadDepth - PrimitiveVolumeBounds.w);

	// Clamp to valid range, start at first slice for the current pass
	int ClosestSliceIndex = max(ClosestSliceIndexUnclamped, 0) + MAX_SLICES_PER_VOXELIZATION_PASS * VoxelizationPassIndex;
	// Clamp to valid range, end at the last slice for the current pass
	int FurthestSliceIndex = min(min(FurthestSliceIndexUnclamped, GridSizeZ - 1), ClosestSliceIndex + MAX_SLICES_PER_VOXELIZATION_PASS - 1);
	
	if (ClosestSliceIndexUnclamped < GridSizeZ
		&& ClosestSliceIndex <= FurthestSliceIndexUnclamped
		&& FurthestSliceIndexUnclamped >= 0)
	{
#if VOXELIZE_SHAPE_MODE == PRIMITIVE_SPHERE_MODE

		float InvPrimitiveVolumeRadius = 1.0f / PrimitiveVolumeBounds.w;
		float2 PrimitiveCenterToVertex[NUM_INPUT_VERTICES];

		{
			UNROLL
			for (uint VertexIndex = 0; VertexIndex < NUM_INPUT_VERTICES; VertexIndex++)
			{
				PrimitiveCenterToVertex[VertexIndex] = (Inputs[VertexIndex].VertexQuadCoordinate - ViewSpacePrimitiveVolumeOrigin.xy) * InvPrimitiveVolumeRadius;
			}
		}

#elif VOXELIZE_SHAPE_MODE == OBJECT_BOX_MODE
		float2 ViewSpaceBoundingBoxMin = 10000000;
		float2 ViewSpaceBoundingBoxMax = -10000000;

		for (int BoxVertex = 0; BoxVertex < 8; BoxVertex++)
		{
			float3 BoxVertexMask = float3((BoxVertex >> 0) & 1, (BoxVertex >> 1) & 1, (BoxVertex >> 2) & 1);
			float3 BoxVertexLocalPosition = LocalObjectBoundsMin + (LocalObjectBoundsMax - LocalObjectBoundsMin) * BoxVertexMask;
			float3 BoxVertexPosition = DFTransformLocalToTranslatedWorld(BoxVertexLocalPosition, LocalToWorld, ResolvedView.PreViewTranslation).xyz;
			float3 ViewSpaceBoxVertex = mul(float4(BoxVertexPosition, 1), ResolvedView.TranslatedWorldToView).xyz;
			ViewSpaceBoundingBoxMin = min(ViewSpaceBoundingBoxMin, ViewSpaceBoxVertex.xy);
			ViewSpaceBoundingBoxMax = max(ViewSpaceBoundingBoxMax, ViewSpaceBoxVertex.xy);
		}
#endif

		// Clone the triangle to each slice
		for (int SliceIndex = ClosestSliceIndex; SliceIndex <= FurthestSliceIndex; SliceIndex++)
		{
			float SliceDepth = ComputeDepthFromZSlice(SliceIndex + VoxelizeVolumePass.FrameJitterOffset0.z);
			float SliceDepthOffset = abs(SliceDepth - ViewSpacePrimitiveVolumeOrigin.z);

			BRANCH
			if (PrimitiveVolumeBounds.w == 0 || SliceDepthOffset <= PrimitiveVolumeBounds.w)
			{
				float SliceRadius = sqrt(max(PrimitiveVolumeBounds.w * PrimitiveVolumeBounds.w - SliceDepthOffset * SliceDepthOffset, 0.0f));

				float2 ViewSpaceVertices[NUM_INPUT_VERTICES];

				{
					UNROLL
					for (uint VertexIndex = 0; VertexIndex < NUM_INPUT_VERTICES; VertexIndex++)
					{
						#if VOXELIZE_SHAPE_MODE == PRIMITIVE_SPHERE_MODE
							// Use vertex positions that bound this slice of the sphere in view space
							ViewSpaceVertices[VertexIndex] = ViewSpacePrimitiveVolumeOrigin.xy + PrimitiveCenterToVertex[VertexIndex] * SliceRadius;
						#elif VOXELIZE_SHAPE_MODE == OBJECT_BOX_MODE
							// Use vertex positions that bound the OBB in view space
							//@todo - accurately intersect each slice with the OBB producing a 3-6 vertex polygon like "A Vertex Program for Efficient Box-Plane Intersection"
							float2 CornerMask = Inputs[VertexIndex].VertexQuadCoordinate;
							ViewSpaceVertices[VertexIndex] = ViewSpaceBoundingBoxMin + (ViewSpaceBoundingBoxMax - ViewSpaceBoundingBoxMin) * CornerMask;
						#endif
					}
				}

				{
					UNROLL
					for (uint VertexIndex = 0; VertexIndex < NUM_INPUT_VERTICES; VertexIndex++)
					{
#if !USING_VERTEX_SHADER_LAYER
						FVoxelizeVolumePrimitiveGSToPS Output;
#endif
						Output.SliceIndex = SliceIndex;

						float3 ViewSpaceVertexPosition = float3(ViewSpaceVertices[VertexIndex], SliceDepth);
						Output.OutPosition = mul(float4(ViewSpaceVertexPosition, 1), VoxelizeVolumePass.ViewToVolumeClip);
						Output.FactoryInterpolants = Inputs[VertexIndex].FactoryInterpolants;

#if !USING_VERTEX_SHADER_LAYER
						OutStream.Append(Output);
#endif
					}
				}

#if !USING_VERTEX_SHADER_LAYER
				OutStream.RestartStrip();
#endif
			}
		}
	}
}

void VoxelizeVS(
	FVertexFactoryInput Input,
#if USING_VERTEX_SHADER_LAYER
	out FVoxelizeVolumePrimitiveGSToPS Output
#else
	out FVoxelizeVolumePrimitiveVSToGS Output
#endif
)
{
	ResolvedView = ResolveView();

	FVertexFactoryIntermediates VFIntermediates = GetVertexFactoryIntermediates(Input);
	float4 TranslatedWorldPosition = VertexFactoryGetWorldPosition(Input, VFIntermediates);
	float3x3 TangentToLocal = VertexFactoryGetTangentToLocal(Input, VFIntermediates);

	// Warning: with OBJECT_BOX_MODE enabled, TranslatedWorldPosition here is wrong.  Any VF interpolants based on it which are passed to other shader stages will also be wrong.
	FMaterialVertexParameters VertexParameters = GetMaterialVertexParameters(Input, VFIntermediates, TranslatedWorldPosition.xyz, TangentToLocal);
	Output.FactoryInterpolants = VertexFactoryGetInterpolantsVSToPS(Input, VFIntermediates, VertexParameters);

	float3 ViewSpacePosition = mul(TranslatedWorldPosition, ResolvedView.TranslatedWorldToView).xyz;

#if USING_VERTEX_SHADER_LAYER
	FVoxelizeVolumePrimitiveVSToGS Inputs[1];
	Inputs[0].FactoryInterpolants = Output.FactoryInterpolants;

	#if VOXELIZE_SHAPE_MODE == PRIMITIVE_SPHERE_MODE
		Inputs[0].VertexQuadCoordinate = ViewSpacePosition.xy;
	#elif VOXELIZE_SHAPE_MODE == OBJECT_BOX_MODE
		Inputs[0].VertexQuadCoordinate = float2(Input.VertexId & 1, (Input.VertexId >> 1) & 1);
	#endif

	Voxelize(Inputs, Output);
#else
	#if VOXELIZE_SHAPE_MODE == PRIMITIVE_SPHERE_MODE
		Output.VertexQuadCoordinate = ViewSpacePosition.xy;
	#elif VOXELIZE_SHAPE_MODE == OBJECT_BOX_MODE
		Output.VertexQuadCoordinate = float2(Input.VertexId & 1, (Input.VertexId >> 1) & 1);
	#endif
#endif
}

float ComputeVolumeShapeMasking(float3 TranslatedWorldPosition, uint PrimitiveId, FVertexFactoryInterpolantsVSToPS FactoryInterpolants)
{
	float ValidRegionMask = 1;

#if VOXELIZE_SHAPE_MODE == PRIMITIVE_SPHERE_MODE

	float4 PrimitiveVolumeBounds = VertexFactoryGetTranslatedPrimitiveVolumeBounds(FactoryInterpolants);
	float3 ConnectingVector = PrimitiveVolumeBounds.xyz - TranslatedWorldPosition;

	FLATTEN
	if (dot(ConnectingVector, ConnectingVector) > PrimitiveVolumeBounds.w * PrimitiveVolumeBounds.w)
	{
		ValidRegionMask = 0;
	}

#elif VOXELIZE_SHAPE_MODE == OBJECT_BOX_MODE

	float3 LocalObjectBoundsMin = GetPrimitiveData(PrimitiveId).LocalObjectBoundsMin;
	float3 LocalObjectBoundsMax = GetPrimitiveData(PrimitiveId).LocalObjectBoundsMax;

	float4x4 TranslatedWorldToLocal = DFFastToTranslatedWorld(GetPrimitiveData(PrimitiveId).WorldToLocal, ResolvedView.PreViewTranslation);
	float3 LocalPosition = mul(float4(TranslatedWorldPosition, 1), TranslatedWorldToLocal).xyz;

	FLATTEN
	if (any((LocalPosition < LocalObjectBoundsMin || LocalPosition > LocalObjectBoundsMax)))
	{
		ValidRegionMask = 0;
	}

#endif

	return ValidRegionMask;
}

void VoxelizePS(
	FVoxelizeVolumePrimitiveGSToPS Interpolants,
	out float4 OutVBufferA : SV_Target0,
	out float4 OutVBufferB : SV_Target1
	)
{
	float4 SvPosition = Interpolants.OutPosition;

	ResolvedView = ResolveView();

	float ValidRegionMask = 1;

	FMaterialPixelParameters MaterialParameters = GetMaterialPixelParameters(Interpolants.FactoryInterpolants, SvPosition);

#if CLOUD_LAYER_PIXEL_SHADER
	FCloudLayerParameters CloudLayerParams = GetCloudLayerParams(
		VoxelizeVolumePass.RenderVolumetricCloudParametersCloudLayerCenterKm, VoxelizeVolumePass.RenderVolumetricCloudParametersPlanetRadiusKm,
		VoxelizeVolumePass.RenderVolumetricCloudParametersBottomRadiusKm, VoxelizeVolumePass.RenderVolumetricCloudParametersTopRadiusKm);
	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(SvPosition);
	// WorldPositionDDX/Y only need to be set when using analytical derivatives, which is not needed in pixel shaders
	UpdateMaterialCloudParam(MaterialParameters, TranslatedWorldPosition, 0.0f /*WorldPositionDDX*/, 0.0f /*WorldPositionDDY*/, ResolvedView, CloudLayerParams, TRACING_SHADOW_DISTANCE_OFF, SPACE_SKIPPING_SLICE_DEPTH_OFF);
#endif

	// Scale rendered position to account for frustum difference between fog voxelization and rendered scene
	float4 AdjustedSvPosition = SvPosition * float4(VoxelizeVolumePass.ClipRatio, 1, 1);

	FPixelMaterialInputs PixelMaterialInputs;
	CalcMaterialParameters(MaterialParameters, PixelMaterialInputs, AdjustedSvPosition, true);

	float3 EmissiveColor = SampleEmissive(PixelMaterialInputs);
	float3 Albedo = SampleAlbedo(PixelMaterialInputs, ResolvedView);
	float Extinction = SampleExtinctionCoefficients(PixelMaterialInputs);

	float3 Scattering = Albedo * Extinction;

	float FadeStart = .6f;
	float SliceFadeAlpha = 1 - saturate((SvPosition.w / VolumetricFog.MaxDistance - FadeStart) / (1 - FadeStart));

	float UnitScale = 1.0f / 100.0f;
	float Scale = UnitScale * SliceFadeAlpha * SliceFadeAlpha * SliceFadeAlpha;

	// Would be faster to branch around the whole material evaluation, but that would cause divergent flow control for gradient operations
	Scale *= ComputeVolumeShapeMasking(MaterialParameters.WorldPosition_CamRelative, MaterialParameters.PrimitiveId, Interpolants.FactoryInterpolants);

#if MATERIAL_VIRTUALTEXTURE_FEEDBACK
	FinalizeVirtualTextureFeedback(MaterialParameters.VirtualTextureFeedback, SvPosition, View.VTFeedbackBuffer);
#endif

	OutVBufferA = float4(Scattering * Scale, Extinction * Scale);
	OutVBufferB = float4(EmissiveColor * Scale, 0);
}
