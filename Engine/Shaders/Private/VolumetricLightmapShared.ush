// Copyright Epic Games, Inc. All Rights Reserved.

#if SUPPORTS_INDEPENDENT_SAMPLERS
	#define PIVSharedAmbientSampler View.SharedBilinearClampedSampler
	#define PIVSharedSampler0 View.SharedBilinearClampedSampler
	#define PIVSharedSampler1 View.SharedBilinearClampedSampler
	#define PIVSharedSampler2 View.SharedBilinearClampedSampler
	#define PIVSharedSampler3 View.SharedBilinearClampedSampler
	#define PIVSharedSampler4 View.SharedBilinearClampedSampler
	#define PIVSharedSampler5 View.SharedBilinearClampedSampler
	#define VLDirectionalLightShadowingSampler View.SharedBilinearClampedSampler
	#define VLSkyBentNormalSharedSampler View.SharedBilinearClampedSampler
#else
	#define PIVSharedAmbientSampler View.VolumetricLightmapBrickAmbientVectorSampler
	#define PIVSharedSampler0 View.VolumetricLightmapTextureSampler0
	#define PIVSharedSampler1 View.VolumetricLightmapTextureSampler1
	#define PIVSharedSampler2 View.VolumetricLightmapTextureSampler2
	#define PIVSharedSampler3 View.VolumetricLightmapTextureSampler3
	#define PIVSharedSampler4 View.VolumetricLightmapTextureSampler4
	#define PIVSharedSampler5 View.VolumetricLightmapTextureSampler5
	#define VLDirectionalLightShadowingSampler View.DirectionalLightShadowingTextureSampler
	#define VLSkyBentNormalSharedSampler View.SkyBentNormalTextureSampler
#endif

float3 ComputeVolumetricLightmapBrickTextureUVs(float3 WorldPosition) // RT_LWC_TODO
{
	// Compute indirection UVs from world position
	float3 IndirectionVolumeUVs = clamp(WorldPosition * View.VolumetricLightmapWorldToUVScale + View.VolumetricLightmapWorldToUVAdd, 0.0f, .99f);
	float3 IndirectionTextureTexelCoordinate = IndirectionVolumeUVs * View.VolumetricLightmapIndirectionTextureSize;
	float4 BrickOffsetAndSize = View.VolumetricLightmapIndirectionTexture.Load(int4(IndirectionTextureTexelCoordinate, 0));

	float PaddedBrickSize = View.VolumetricLightmapBrickSize + 1;
	return (BrickOffsetAndSize.xyz * PaddedBrickSize + frac(IndirectionTextureTexelCoordinate / BrickOffsetAndSize.w) * View.VolumetricLightmapBrickSize + .5f) * View.VolumetricLightmapBrickTexelSize;
}

float3 GetVolumetricLightmapAmbient(float3 BrickTextureUVs)
{
	return Texture3DSampleLevel(View.VolumetricLightmapBrickAmbientVector, PIVSharedAmbientSampler, BrickTextureUVs, 0).xyz;
}

FOneBandSHVectorRGB GetVolumetricLightmapSH1(float3 BrickTextureUVs)
{
	float3 AmbientVector = GetVolumetricLightmapAmbient(BrickTextureUVs);

	FOneBandSHVectorRGB IrradianceSH;
	IrradianceSH.R.V = AmbientVector.x;
	IrradianceSH.G.V = AmbientVector.y;
	IrradianceSH.B.V = AmbientVector.z;

	return IrradianceSH;
}

void GetVolumetricLightmapSHCoefficients0(float3 BrickTextureUVs, out float3 AmbientVector, out float4 SHCoefficients0Red, out float4 SHCoefficients0Green, out float4 SHCoefficients0Blue)
{
	AmbientVector = GetVolumetricLightmapAmbient(BrickTextureUVs);
	SHCoefficients0Red = Texture3DSampleLevel(View.VolumetricLightmapBrickSHCoefficients0, PIVSharedSampler0, BrickTextureUVs, 0) * 2 - 1;
	SHCoefficients0Green = Texture3DSampleLevel(View.VolumetricLightmapBrickSHCoefficients2, PIVSharedSampler2, BrickTextureUVs, 0) * 2 - 1;
	SHCoefficients0Blue = Texture3DSampleLevel(View.VolumetricLightmapBrickSHCoefficients4, PIVSharedSampler4, BrickTextureUVs, 0) * 2 - 1;

	// Undo normalization done in FIrradianceBrickData::SetFromVolumeLightingSample
	float4 SHDenormalizationScales0 = float4(
		0.488603f / 0.282095f, 
		0.488603f / 0.282095f, 
		0.488603f / 0.282095f, 
		1.092548f / 0.282095f);

	SHCoefficients0Red = SHCoefficients0Red * AmbientVector.x * SHDenormalizationScales0;
	SHCoefficients0Green = SHCoefficients0Green * AmbientVector.y * SHDenormalizationScales0;
	SHCoefficients0Blue = SHCoefficients0Blue * AmbientVector.z * SHDenormalizationScales0;
}

FTwoBandSHVectorRGB GetVolumetricLightmapSH2(float3 BrickTextureUVs)
{
	float3 AmbientVector;
	float4 SHCoefficients0Red;
	float4 SHCoefficients0Green;
	float4 SHCoefficients0Blue;
	GetVolumetricLightmapSHCoefficients0(BrickTextureUVs, AmbientVector, SHCoefficients0Red, SHCoefficients0Green, SHCoefficients0Blue);

	FTwoBandSHVectorRGB IrradianceSH;

	IrradianceSH.R.V = float4(AmbientVector.x, SHCoefficients0Red.xyz);
	IrradianceSH.G.V = float4(AmbientVector.y, SHCoefficients0Green.xyz);
	IrradianceSH.B.V = float4(AmbientVector.z, SHCoefficients0Blue.xyz);

	return IrradianceSH;
}

FThreeBandSHVectorRGB GetVolumetricLightmapSH3(float3 BrickTextureUVs)
{
	float3 AmbientVector;
	float4 SHCoefficients0Red;
	float4 SHCoefficients0Green;
	float4 SHCoefficients0Blue;
	GetVolumetricLightmapSHCoefficients0(BrickTextureUVs, AmbientVector, SHCoefficients0Red, SHCoefficients0Green, SHCoefficients0Blue);

	float4 SHCoefficients1Red = Texture3DSampleLevel(View.VolumetricLightmapBrickSHCoefficients1, PIVSharedSampler1, BrickTextureUVs, 0) * 2 - 1;
	float4 SHCoefficients1Green = Texture3DSampleLevel(View.VolumetricLightmapBrickSHCoefficients3, PIVSharedSampler3, BrickTextureUVs, 0) * 2 - 1;
	float4 SHCoefficients1Blue = Texture3DSampleLevel(View.VolumetricLightmapBrickSHCoefficients5, PIVSharedSampler5, BrickTextureUVs, 0) * 2 - 1;

	float4 SHDenormalizationScales1 = float4(
		1.092548f / 0.282095f,
		4.0f * 0.315392f / 0.282095f,
		1.092548f / 0.282095f,
		2.0f * 0.546274f / 0.282095f);

	SHCoefficients1Red = SHCoefficients1Red * AmbientVector.x * SHDenormalizationScales1;
	SHCoefficients1Green = SHCoefficients1Green * AmbientVector.y * SHDenormalizationScales1;
	SHCoefficients1Blue = SHCoefficients1Blue * AmbientVector.z * SHDenormalizationScales1;

	FThreeBandSHVectorRGB IrradianceSH;
	// Construct the SH environment
	IrradianceSH.R.V0 = float4(AmbientVector.x, SHCoefficients0Red.xyz);
	IrradianceSH.R.V1 = float4(SHCoefficients0Red.w, SHCoefficients1Red.xyz);
	IrradianceSH.R.V2 = SHCoefficients1Red.w;

	IrradianceSH.G.V0 = float4(AmbientVector.y, SHCoefficients0Green.xyz);
	IrradianceSH.G.V1 = float4(SHCoefficients0Green.w, SHCoefficients1Green.xyz);
	IrradianceSH.G.V2 = SHCoefficients1Green.w;

	IrradianceSH.B.V0 = float4(AmbientVector.z, SHCoefficients0Blue.xyz);
	IrradianceSH.B.V1 = float4(SHCoefficients0Blue.w, SHCoefficients1Blue.xyz);
	IrradianceSH.B.V2 = SHCoefficients1Blue.w;

	return IrradianceSH;
}

float3 GetVolumetricLightmapSkyBentNormal(float3 BrickTextureUVs)
{
	float3 SkyBentNormal = Texture3DSampleLevel(View.SkyBentNormalBrickTexture, VLSkyBentNormalSharedSampler, BrickTextureUVs, 0).xyz * 2 - 1;
	return SkyBentNormal;
}

float GetVolumetricLightmapDirectionalLightShadowing(float3 BrickTextureUVs)
{
	return Texture3DSampleLevel(View.DirectionalLightShadowingBrickTexture, VLDirectionalLightShadowingSampler, BrickTextureUVs, 0).x;
}