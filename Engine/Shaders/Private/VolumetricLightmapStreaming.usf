// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	VolumetricLightmapStreaming.usf
=============================================================================*/

#include "/Engine/Public/Platform.ush"

uint3 ComputeBrickLayoutPosition(int BrickLayoutAllocation, uint3 BrickLayoutDimensions)
{
	uint3 BrickPosition = uint3(
		(BrickLayoutAllocation % BrickLayoutDimensions.x),
		(BrickLayoutAllocation / BrickLayoutDimensions.x) % BrickLayoutDimensions.y,
		(BrickLayoutAllocation / BrickLayoutDimensions.x) / BrickLayoutDimensions.y);

	return BrickPosition;
}

uint NumBricks;

uint PersistentLevelBrickDataBaseOffset;
Buffer<uint> SubLevelBrickPositions;
Buffer<uint4> IndirectionTextureOriginalValues;
RWTexture3D<uint4> IndirectionTexture;
		
[numthreads(64, 1, 1)]
void RemoveSubLevelBricksCS(uint3 BrickID : SV_DispatchThreadID) 
{
	if (BrickID.x >= NumBricks) return;

	// Restore indirection texture value to the persistent level one
	uint3 BrickPos = uint3(
		SubLevelBrickPositions[BrickID.x * 3 + 0],
		SubLevelBrickPositions[BrickID.x * 3 + 1],
		SubLevelBrickPositions[BrickID.x * 3 + 2]
	);
	
	// IndirectionTextureOriginalValues is stored as an FColor which is a bgra memory layout so we swizzle on read to get the correct values
	uint4 IndirectionTextureOriginalValue = IndirectionTextureOriginalValues[BrickID.x].bgra;	
	uint OriginalBrickLinearAddress = IndirectionTextureOriginalValue.x + IndirectionTextureOriginalValue.y * 256 + IndirectionTextureOriginalValue.z * 256 * 256;
	IndirectionTexture[BrickPos] = uint4(ComputeBrickLayoutPosition(OriginalBrickLinearAddress + PersistentLevelBrickDataBaseOffset, uint3(256, 256, 256)), IndirectionTextureOriginalValue.w);
}

Texture3D AmbientVector;
Texture3D SHCoefficients;
Texture3D SkyBentNormal;
Texture3D DirectionalLightShadowing;

RWTexture3D<float3> OutAmbientVector;
RWTexture3D<UNORM float4> OutSHCoefficients;
RWTexture3D<UNORM float4> OutSkyBentNormal;
RWTexture3D<UNORM float> OutDirectionalLightShadowing;

uint StartPosInOldVolume;
uint StartPosInNewVolume;

[numthreads(5, 5, 5)]
void CopyResidentBricksCS(
	uint3 BrickID : SV_GroupID,
	uint3 CellPos : SV_GroupThreadID) 
{
	const uint PaddedSize = 5;
	uint3 OldBrickPos = ComputeBrickLayoutPosition(StartPosInOldVolume + BrickID.x, uint3(256, 256, 256));
	uint3 OldPos = OldBrickPos * PaddedSize + CellPos;
	uint3 NewBrickPos = ComputeBrickLayoutPosition(StartPosInNewVolume + BrickID.x, uint3(256, 256, 256));
	uint3 NewPos = NewBrickPos * PaddedSize + CellPos;
	
	OutAmbientVector[NewPos] = AmbientVector[OldPos].xyz;
	#if HAS_SKY_BENT_NORMAL
		OutSkyBentNormal[NewPos] = SkyBentNormal[OldPos];
	#endif
	OutDirectionalLightShadowing[NewPos] = DirectionalLightShadowing[OldPos].x;
}

[numthreads(5, 5, 5)]
void CopyResidentBrickSHCoefficientsCS(
	uint3 BrickID : SV_GroupID,
	uint3 CellPos : SV_GroupThreadID) 
{
	const uint PaddedSize = 5;
	uint3 OldBrickPos = ComputeBrickLayoutPosition(StartPosInOldVolume + BrickID.x, uint3(256, 256, 256));
	uint3 OldPos = OldBrickPos * PaddedSize + CellPos;
	uint3 NewBrickPos = ComputeBrickLayoutPosition(StartPosInNewVolume + BrickID.x, uint3(256, 256, 256));
	uint3 NewPos = NewBrickPos * PaddedSize + CellPos;

	OutSHCoefficients[NewPos] = SHCoefficients[OldPos];
}

[numthreads(64, 1, 1)]
void PatchIndirectionTextureCS(uint3 BrickID : SV_DispatchThreadID) 
{
	if (BrickID.x >= NumBricks) return;

	const uint PaddedSize = 5;
	uint3 NewBrickPos = ComputeBrickLayoutPosition(StartPosInNewVolume + BrickID.x, uint3(256, 256, 256));

	// Patch indirection texture to new position
	uint3 BrickPos = uint3(
		SubLevelBrickPositions[BrickID.x * 3 + 0],
		SubLevelBrickPositions[BrickID.x * 3 + 1],
		SubLevelBrickPositions[BrickID.x * 3 + 2]
	);
	
	IndirectionTexture[BrickPos] = uint4(NewBrickPos, 1);
}

Texture3D<uint4> OldIndirectionTexture;

[numthreads(4, 4, 4)]
void MoveWholeIndirectionTextureCS(uint3 IndirectionTextureCellID : SV_DispatchThreadID) 
{
	uint3 IndirectionTextureDimenstions;
	IndirectionTexture.GetDimensions(IndirectionTextureDimenstions.x, IndirectionTextureDimenstions.y, IndirectionTextureDimenstions.z);
	
	if (any(IndirectionTextureCellID >= IndirectionTextureDimenstions)) return;

	uint4 IndirectionTextureOriginalValue = OldIndirectionTexture[IndirectionTextureCellID];
	uint OriginalBrickLinearAddress = IndirectionTextureOriginalValue.x + IndirectionTextureOriginalValue.y * 256 + IndirectionTextureOriginalValue.z * 256 * 256;
	
	if (OriginalBrickLinearAddress >= StartPosInOldVolume && OriginalBrickLinearAddress < StartPosInOldVolume + NumBricks)
	{
		IndirectionTexture[IndirectionTextureCellID] = uint4(ComputeBrickLayoutPosition(OriginalBrickLinearAddress - StartPosInOldVolume + StartPosInNewVolume, uint3(256, 256, 256)), IndirectionTextureOriginalValue.w);
	}
	else
	{
		IndirectionTexture[IndirectionTextureCellID] = IndirectionTextureOriginalValue;
	}
}