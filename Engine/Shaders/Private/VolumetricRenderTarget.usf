// Copyright Epic Games, Inc. All Rights Reserved.

/**
 * VolumetricRenderTarget.usf: all the necessary processes required to temporally reconstruct the volumetric render target.
 */

#include "Common.ush"
#include "Random.ush"

#include "SceneTextureParameters.ush"

#include "Substrate/Substrate.ush"

#ifndef CLOUD_MIN_AND_MAX_DEPTH
#define CLOUD_MIN_AND_MAX_DEPTH 0
#endif



#define APPLY_FOG_LATE (PERMUTATION_APPLY_FOG || PERMUTATION_APPLY_LOCAL_FOG_VOLUME)

#if PERMUTATION_APPLY_FOG
#include "HeightFogCommon.ush"
#endif // PERMUTATION_APPLY_FOG

#if PERMUTATION_APPLY_LOCAL_FOG_VOLUME
#include "LocalFogVolumes/LocalFogVolumeCommon.ush"
#endif // PERMUTATION_SUPPORT_LOCAL_FOG_VOLUME

// BufferSVPos is needed when using view matrices operation relying on VolumetricRenderTargetViewUniformBuffer such as GetScreenWorldDir.
float4 ApplyFogToCloudSampleFromVPosAndDepth(float2 FullResViewSVPos, float2 BufferSVPos, float DepthKilometer, float4 CloudRGBT, float2 View_ViewRectMin)
{
#if APPLY_FOG_LATE

	const float CloudCoverage = saturate(1.0 - CloudRGBT.a);
	if (CloudCoverage <= 0.0f)
	{
		return CloudRGBT;
	}

	float4 HeightFogInscatteringAndTransmittance = float4(0, 0, 0, 1);
	const float DistanceToPoint = DepthKilometer * KILOMETER_TO_CENTIMETER;

	const float3 FogSampleWorldPositionRelativeToCameraCm = normalize(GetScreenWorldDir(float4(BufferSVPos, 0.5f, 1.0f))) * DistanceToPoint; // bug have to div by 4

#if PERMUTATION_APPLY_FOG
	{
		HeightFogInscatteringAndTransmittance = CalculateHeightFog(FogSampleWorldPositionRelativeToCameraCm);
	}
#endif

#if PERMUTATION_APPLY_LOCAL_FOG_VOLUME
	uint2 TilePos = (FullResViewSVPos - View_ViewRectMin) / LFVTilePixelSize.xx; // Remove view rect min as we take the view pixel to sample the tiled LFV culling result

	float4 LFVContribution = GetLFVContribution(PrimaryView, TilePos, FogSampleWorldPositionRelativeToCameraCm);
	if (LFVRenderInVolumetricFog > 0)
	{
		HeightFogInscatteringAndTransmittance = float4(LFVContribution.rgb + HeightFogInscatteringAndTransmittance.rgb * LFVContribution.a, LFVContribution.a * HeightFogInscatteringAndTransmittance.a);
	}
#endif

#if PERMUTATION_APPLY_FOG
	if (FogStruct.ApplyVolumetricFog > 0)
	{
		float4 ClipPos = mul(float4(FogSampleWorldPositionRelativeToCameraCm, 1.0f), PrimaryView.TranslatedWorldToClip);
		float3 VolumeUV = ComputeVolumeUVFromNDC(ClipPos);
		const uint EyeIndex = 0;
		HeightFogInscatteringAndTransmittance = CombineVolumetricFog(HeightFogInscatteringAndTransmittance, VolumeUV, EyeIndex, DistanceToPoint);
	}
#endif

#if PERMUTATION_APPLY_LOCAL_FOG_VOLUME
	if (LFVRenderInVolumetricFog == 0)
	{
		HeightFogInscatteringAndTransmittance = float4(LFVContribution.rgb + HeightFogInscatteringAndTransmittance.rgb * LFVContribution.a, LFVContribution.a * HeightFogInscatteringAndTransmittance.a);
	}
#endif

	HeightFogInscatteringAndTransmittance.rgb *= PrimaryView.PreExposure;

	CloudRGBT.rgb = CloudRGBT.rgb * HeightFogInscatteringAndTransmittance.a + CloudCoverage * HeightFogInscatteringAndTransmittance.rgb;
#endif // APPLY_FOG_LATE

	return CloudRGBT;
}






#ifdef SHADER_RECONSTRUCT_VOLUMETRICRT

SamplerState LinearTextureSampler;

Texture2D HalfResDepthTexture;

Texture2D<float4> TracingVolumetricTexture;
Texture2D<float4> SecondaryTracingVolumetricTexture;
Texture2D<float4> TracingVolumetricDepthTexture;
uint4 TracingVolumetricTextureValidCoordRect;
float4 TracingVolumetricTextureValidUvRect;
float MinimumDistanceKmToEnableReprojection;
float MinimumDistanceKmToDisableDisoclusion;
float2 TracingVolumetricTextureUVScale;

struct FDepthData
{
	float  CloudFrontDepthFromViewKm;
	float  SceneDepthFromViewKm;
	float2 MinMaxViewDepthKm;
};
FDepthData GetDepthDataFromVector(float4 DepthVector)
{
	FDepthData DepthData;
	DepthData.CloudFrontDepthFromViewKm = DepthVector.x;
	DepthData.SceneDepthFromViewKm		= DepthVector.y;
	DepthData.MinMaxViewDepthKm			= DepthVector.zw;
	return DepthData;
}

float4 SafeLoadTracingVolumetricTexture(uint2 Coord)
{
	return TracingVolumetricTexture.Load(uint3(clamp(Coord, TracingVolumetricTextureValidCoordRect.xy, TracingVolumetricTextureValidCoordRect.zw), 0));
}
float4 SafeSampleTracingVolumetricTexture(float2 UV)
{
	UV *= TracingVolumetricTextureUVScale.xy;
	return TracingVolumetricTexture.SampleLevel(LinearTextureSampler, clamp(UV, TracingVolumetricTextureValidUvRect.xy, TracingVolumetricTextureValidUvRect.zw), 0);
}
float4 SafeLoadSecondaryTracingVolumetricTexture(uint2 Coord)
{
	return SecondaryTracingVolumetricTexture.Load(uint3(clamp(Coord, TracingVolumetricTextureValidCoordRect.xy, TracingVolumetricTextureValidCoordRect.zw), 0));
}
float4 SafeSampleSecondaryTracingVolumetricTexture(float2 UV)
{
	UV *= TracingVolumetricTextureUVScale.xy;
	return SecondaryTracingVolumetricTexture.SampleLevel(LinearTextureSampler, clamp(UV, TracingVolumetricTextureValidUvRect.xy, TracingVolumetricTextureValidUvRect.zw), 0);
}
FDepthData SafeLoadTracingVolumetricDepthTexture(uint2 Coord)
{
	return GetDepthDataFromVector(TracingVolumetricDepthTexture.Load(uint3(clamp(Coord, TracingVolumetricTextureValidCoordRect.xy, TracingVolumetricTextureValidCoordRect.zw), 0)));
}
FDepthData SafeSampleTracingVolumetricDepthTexture(float2 UV)
{
	UV *= TracingVolumetricTextureUVScale.xy;
	return GetDepthDataFromVector(TracingVolumetricDepthTexture.SampleLevel(LinearTextureSampler, clamp(UV, TracingVolumetricTextureValidUvRect.xy, TracingVolumetricTextureValidUvRect.zw), 0));
}

#if PERMUTATION_HISTORY_AVAILABLE
Texture2D<float4> PreviousFrameVolumetricTexture; 
Texture2D<float4> PreviousFrameVolumetricSecondaryTexture;
Texture2D<float4> PreviousFrameVolumetricDepthTexture;
float4 PreviousVolumetricTextureSizeAndInvSize;

uint4 PreviousFrameVolumetricTextureValidCoordRect;
float4 PreviousFrameVolumetricTextureValidUvRect;
float2 PreviousFrameVolumetricTextureUVScale;

float HistoryPreExposureCorrection;

float4 SafeLoadPreviousFrameVolumetricTexture(uint2 Coord)
{
	float4 Result = PreviousFrameVolumetricTexture.Load(uint3(clamp(Coord, PreviousFrameVolumetricTextureValidCoordRect.xy, PreviousFrameVolumetricTextureValidCoordRect.zw), 0));
	return float4(Result.rgb * HistoryPreExposureCorrection, Result.a);
}
float4 SafeSamplePreviousFrameVolumetricTexture(float2 UV)
{
	UV *= PreviousFrameVolumetricTextureUVScale.xy;
	float4 Result = PreviousFrameVolumetricTexture.SampleLevel(LinearTextureSampler, clamp(UV, PreviousFrameVolumetricTextureValidUvRect.xy, PreviousFrameVolumetricTextureValidUvRect.zw), 0);
	return float4(Result.rgb * HistoryPreExposureCorrection, Result.a);
}

float4 SafeLoadPreviousFrameVolumetricSecondaryTexture(uint2 Coord)
{
	float4 Result = PreviousFrameVolumetricSecondaryTexture.Load(uint3(clamp(Coord, PreviousFrameVolumetricTextureValidCoordRect.xy, PreviousFrameVolumetricTextureValidCoordRect.zw), 0));
	return float4(Result.rgb * HistoryPreExposureCorrection, Result.a);
}
float4 SafeSamplePreviousFrameVolumetricSecondaryTexture(float2 UV)
{
	UV *= PreviousFrameVolumetricTextureUVScale.xy;
	float4 Result = PreviousFrameVolumetricSecondaryTexture.SampleLevel(LinearTextureSampler, clamp(UV, PreviousFrameVolumetricTextureValidUvRect.xy, PreviousFrameVolumetricTextureValidUvRect.zw), 0);
	return float4(Result.rgb * HistoryPreExposureCorrection, Result.a);
}

FDepthData SafeLoadPreviousFrameVolumetricDepthTexture(uint2 Coord)
{
	return GetDepthDataFromVector(PreviousFrameVolumetricDepthTexture.Load(uint3(clamp(Coord, PreviousFrameVolumetricTextureValidCoordRect.xy, PreviousFrameVolumetricTextureValidCoordRect.zw), 0)));
}
FDepthData SafeSamplePreviousFrameVolumetricDepthTexture(float2 UV)
{
	UV *= PreviousFrameVolumetricTextureUVScale.xy;
	return GetDepthDataFromVector(PreviousFrameVolumetricDepthTexture.SampleLevel(LinearTextureSampler, clamp(UV, PreviousFrameVolumetricTextureValidUvRect.xy, PreviousFrameVolumetricTextureValidUvRect.zw), 0));
}
#endif // PERMUTATION_HISTORY_AVAILABLE

float4 DstVolumetricTextureSizeAndInvSize;
int2 CurrentTracingPixelOffset;
int2 ViewViewRectMin;
int VolumetricRenderTargetMode;
int DownSampleFactor;
float2 SourceViewRectMin;
float2 SourceViewRectSize;

#define USE_YCOCG 0

float3 RGB2CLIP(float3 RGB)
{
#if USE_YCOCG
	return RGBToYCoCg(RGB);
#else
	return RGB;
#endif
}

float3 CLIP2RGB(float3 CLIP)
{
#if USE_YCOCG
	return YCoCgToRGB(CLIP);
#else
	return CLIP;
#endif
}

float BOX_NORM_LUMA(float3 Clip, float3 Min, float3 Max)
{
#if USE_YCOCG
	return saturate((Clip.x - Min.x) / max(0.00001f, Max.x - Min.x));
#else
	float ClipLuma = Luminance(Clip);
	float MinLuma = Luminance(Min);
	float MaxLuma = Luminance(Max);
	return saturate((ClipLuma.x - MinLuma.x) / max(0.00001f, MaxLuma.x - MinLuma.x));
#endif
}

void FixupDepthDataSafe(inout FDepthData DepthData)
{
	// When reprojecting depth, it can happen landscape or other front object leaks into the sky pixels, when those pixel have not been identified as bUseNewSample.
	// This happens rarely but no concrete solution have been found to resolve that situation.
	// To avoid cloud to flicker (depth difference can be large, causing resonstruction to identify a depth discontinuity)
	const float MaxDepthKm = 60000.0f;	// Anything beyond that threshold will be considered at maximum depth MaxHalfFloat.
	DepthData.CloudFrontDepthFromViewKm	= DepthData.CloudFrontDepthFromViewKm	> MaxDepthKm ? MaxHalfFloat : DepthData.CloudFrontDepthFromViewKm;
	DepthData.SceneDepthFromViewKm		= DepthData.SceneDepthFromViewKm		> MaxDepthKm ? MaxHalfFloat : DepthData.SceneDepthFromViewKm;
	DepthData.MinMaxViewDepthKm.x		= DepthData.MinMaxViewDepthKm.x			> MaxDepthKm ? MaxHalfFloat : DepthData.MinMaxViewDepthKm.x;
	DepthData.MinMaxViewDepthKm.y		= DepthData.MinMaxViewDepthKm.y			> MaxDepthKm ? MaxHalfFloat : DepthData.MinMaxViewDepthKm.y;
}

void ReconstructVolumetricRenderTargetPS(
	in float4 SVPos : SV_POSITION, 
#if PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
	out float4 OutputRt0 : SV_Target0,
	out float4 OutputRt1 : SV_Target1,
	out float4 OutputRt2 : SV_Target2
#if APPLY_FOG_LATE
	, out float4 OutputCloudAndFog : SV_Target3
#endif
#else
	out float4 OutputRt0 : SV_Target0,
	out float4 OutputRt1 : SV_Target1
#if APPLY_FOG_LATE
	, out float4 OutputCloudAndFog : SV_Target2
#endif
#endif
)
{
	float2 PixelPos = SVPos.xy;
	float2 ScreenUV = SVPos.xy * DstVolumetricTextureSizeAndInvSize.zw; // UV in [0,1]

#if PERMUTATION_HISTORY_AVAILABLE

	int2 IntPixelPos = int2(PixelPos);
	int2 IntPixelPosDownsample = IntPixelPos / (DownSampleFactor);
	const int XSub = int(IntPixelPos.x) - (IntPixelPosDownsample.x * DownSampleFactor);
	const int YSub = int(IntPixelPos.y) - (IntPixelPosDownsample.y * DownSampleFactor);

	bool bUseNewSample = (XSub == CurrentTracingPixelOffset.x) && (YSub == CurrentTracingPixelOffset.y);
	float4 RGBA = 0.0f;
	float4 RGBA2 = 0.0f;
	FDepthData DepthData = (FDepthData)0;
  
	{
		float2 ScreenPosition = ViewportUVToScreenPos(ScreenUV); // NDC in [-1,1] not using View.ScreenPositionScaleBias here

		// Sample participating media "front depth" for a better reprojection
		float TracingVolumetricSampleDepthKm = SafeLoadTracingVolumetricDepthTexture(int2(SVPos.xy) / DownSampleFactor).CloudFrontDepthFromViewKm;
		float TracingVolumetricSampleDepth = TracingVolumetricSampleDepthKm * KILOMETER_TO_CENTIMETER;
		float DeviceZ = ConvertToDeviceZ(TracingVolumetricSampleDepth); // Approximation. Should try real DeviceZ

		float4 CurrClip = float4(ScreenPosition, DeviceZ, 1); // Inverted Far Depth = 0.0f
		float4 PrevClip = mul(CurrClip, View.ClipToPrevClip);
		float2 PrevScreen = PrevClip.xy / PrevClip.w;
		float2 ScreenVelocity = ScreenPosition - PrevScreen;
		// TODO Sample screen velocity when available

		float2 PrevScreenPosition = (ScreenPosition - ScreenVelocity);   // NDC in [-1,1]
		float2 PrevScreenUVs = ScreenPosToViewportUV(PrevScreenPosition);// UV in [0,1]
		const bool bValidPreviousUVs = all(PrevScreenUVs > 0.0) && all(PrevScreenUVs < 1.0f) 
			&& (TracingVolumetricSampleDepthKm >= MinimumDistanceKmToEnableReprojection);	// This helps hide reprojection issues due to imperfect approximation of cloud depth as a single front surface, 
																							// especially visible when flying through the cloud layer. It is not perfect but will help in lots of cases. 
																							// The problem when using this method: clouds will look noisier when closer to that distance. 


		

	#if 0
		// Debug: always use new sample else history result. Should be 
		if (bUseNewSample)
		{
			RGBA		= SafeLoadTracingVolumetricTexture(int2(SVPos.xy) / DownSampleFactor);
			RGBA2		= SafeLoadSecondaryTracingVolumetricTexture(int2(SVPos.xy) / DownSampleFactor);
			DepthData	= SafeLoadTracingVolumetricDepthTexture(int2(SVPos.xy) / DownSampleFactor);
		}
		else
		{
			RGBA		= SafeSamplePreviousFrameVolumetricTexture(PrevScreenUVs);
			RGBA2		= SafeSamplePreviousFrameVolumetricSecondaryTexture(PrevScreenUVs);
			DepthData	= SafeSamplePreviousFrameVolumetricDepthTexture(PrevScreenUVs);
		}
		OutputRt0 = RGBA;
	#if PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
		OutputRt1 = RGBA2;
		OutputRt2 = float4(
			DepthData.CloudFrontDepthFromViewKm, 
			DepthData.SceneDepthFromViewKm, 
			DepthData.MinMaxViewDepthKm);
	#else
		OutputRt1 = float4(
			DepthData.CloudFrontDepthFromViewKm,
			DepthData.SceneDepthFromViewKm,
			0.0, 0.0);	// unused components
	#endif
		return;
	#endif



		if (VolumetricRenderTargetMode == 2)
		{
			const bool bUseNewSampleMode2 = ((IntPixelPos.x - IntPixelPosDownsample.x * DownSampleFactor) == CurrentTracingPixelOffset.x) && ((IntPixelPos.y - IntPixelPosDownsample.y * DownSampleFactor) == CurrentTracingPixelOffset.y);

			if (bUseNewSampleMode2)
			{
				// Load the new sample for this pixel we have just traced
				RGBA = SafeLoadTracingVolumetricTexture(int2(SVPos.xy) / DownSampleFactor);// +float4(0.1, 0.0, 0, 0);
				DepthData = SafeLoadTracingVolumetricDepthTexture(int2(SVPos.xy) / DownSampleFactor);
			}
			else if(bValidPreviousUVs)
			{
				// Sample valid on screen history using bilinear filtering
				RGBA = SafeSamplePreviousFrameVolumetricTexture(PrevScreenUVs);// +float4(0, 0.1, 0, 0);
				DepthData = SafeSamplePreviousFrameVolumetricDepthTexture(PrevScreenUVs);
			}
			else
			{
				// Bias the sample such that the currently-traced screen pixels line up with the centre of their traced values in the tracing texture
				// This prevents clouds from appearing to shift position when rotating the camera quickly
				float2 ScreenPixelsOffset = -(CurrentTracingPixelOffset - (DownSampleFactor - 1.0f) * 0.5f);
				float2 TracingTextureUV = ScreenUV + ScreenPixelsOffset * DstVolumetricTextureSizeAndInvSize.zw;

				// Sample the new low resolution clouds we just traced using bilinear filtering 
				RGBA = SafeSampleTracingVolumetricTexture(TracingTextureUV);
				DepthData = SafeSampleTracingVolumetricDepthTexture(TracingTextureUV);
			}
		#if PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
			OutputRt0 = float4(0.0f, 0.0f, 0.0f, 1.0f);
			OutputRt1 = float4(0.0f, 0.0f, 0.0f, 1.0f);
			OutputRt2 = float4(63000.0f, 63000.0f, 63000.0f, 63000.0f);
		#else
			OutputRt0 = RGBA;
			OutputRt1 = float4(
				DepthData.CloudFrontDepthFromViewKm, 
				DepthData.SceneDepthFromViewKm,
				0.0, 0.0);	// unused components
		#endif

			#if APPLY_FOG_LATE
			// Apply fog on the a side reconstructed texture if latefog is used for fog on translucent. Otherwise the regular reconstructed cloud texture will be used and this one do not have fog in it.
			float2 FullResViewSVPos = 0*SourceViewRectMin + ScreenUV * TracingVolumetricTextureUVScale.xy * SourceViewRectSize;
			float2 BufferSVPos = FullResViewSVPos / 4.0;
			OutputCloudAndFog = ApplyFogToCloudSampleFromVPosAndDepth(FullResViewSVPos, BufferSVPos, DepthData.CloudFrontDepthFromViewKm, OutputRt0, 0*SourceViewRectMin);
			#endif
			return;
		}
		else
		{

			int2 CenterSample = int2(SVPos.xy) / DownSampleFactor;
			float4 NewRGBA = SafeLoadTracingVolumetricTexture(CenterSample);
			float4 NewRGBA2 = SafeLoadSecondaryTracingVolumetricTexture(CenterSample);
			FDepthData NewDepthData = SafeLoadTracingVolumetricDepthTexture(CenterSample);

			FDepthData HistoryDepthData = SafeSamplePreviousFrameVolumetricDepthTexture(PrevScreenUVs);

			FixupDepthDataSafe(NewDepthData);
			FixupDepthDataSafe(HistoryDepthData);

		#if PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
			
			// Do not apply disoclusion if all the traced and reprojected cloud depth are larger. In this case, cloud information will be like a layer blended on top without upsampling.
			// Otherwise we might be hitting an edge/disoclusion and we will need upsampling.
			const bool bApplyDisoclusion = any(NewDepthData.MinMaxViewDepthKm < MinimumDistanceKmToDisableDisoclusion) || any(HistoryDepthData.MinMaxViewDepthKm < MinimumDistanceKmToDisableDisoclusion)
										|| NewDepthData.SceneDepthFromViewKm < MinimumDistanceKmToDisableDisoclusion || HistoryDepthData.SceneDepthFromViewKm < MinimumDistanceKmToDisableDisoclusion;
			if (bApplyDisoclusion)
			{
				const float ThresholdToNewSampleKm = 2.0; // Arbitrary

				if (abs(NewDepthData.MinMaxViewDepthKm.x - NewDepthData.MinMaxViewDepthKm.y) > ThresholdToNewSampleKm
					&& (abs(NewDepthData.MinMaxViewDepthKm.x - NewDepthData.SceneDepthFromViewKm) < abs(NewDepthData.MinMaxViewDepthKm.y - NewDepthData.SceneDepthFromViewKm)))
				{
					// If there is a huge delta of depth for a pixel, use the new sample but only if we are on the closest depth part.
					// This helps a lot removing cloud over trees and small details
					bUseNewSample = true;
				}
				else
					if (HistoryDepthData.SceneDepthFromViewKm < (NewDepthData.SceneDepthFromViewKm - ThresholdToNewSampleKm))
					{
						// History is closer than the near cloud tracing this frame. This means an object had move and an new disocluded area is discovered.
						// So we simply use the new data from this frame according to the new depth
						bUseNewSample = true;
					}
				else
					if (HistoryDepthData.SceneDepthFromViewKm > (NewDepthData.SceneDepthFromViewKm + ThresholdToNewSampleKm))
					{
						// An area that just go covered (history is invalid because occluded)
						bUseNewSample = true;
					}
			//	else
			//		if (NewDepthData.SceneDepthFromViewKm > 1.5)
			//	{
			//		if (bUseNewSample || !bValidPreviousUVs)
			//		{
			//			RGBA		= SafeLoadTracingVolumetricTexture(int2(SVPos.xy) / DownSampleFactor);
			//			RGBA2		= SafeLoadSecondaryTracingVolumetricTexture(int2(SVPos.xy) / DownSampleFactor);
			//			DepthData	= SafeLoadTracingVolumetricDepthTexture(int2(SVPos.xy) / DownSampleFactor);
			//		}
			//		else
			//		{
			//			RGBA		= SafeSamplePreviousFrameVolumetricTexture(PrevScreenUVs);
			//			RGBA2		= SafeSamplePreviousFrameVolumetricSecondaryTexture(PrevScreenUVs);
			//			DepthData	= SafeSamplePreviousFrameVolumetricDepthTexture(PrevScreenUVs);
			//		}
			//		OutputRt0 = RGBA;// *float4(1, 0, 0, 1);
			//		OutputRt1 = RGBA2;// *float4(1, 0, 0, 1);
			//		OutputRt2 = float4(
			//			DepthData.CloudFrontDepthFromViewKm, 
			//			DepthData.SceneDepthFromViewKm, 
			//			DepthData.MinMaxViewDepthKm);
			//		return;	
			//	}
			}
			

			if (bUseNewSample)
			{
				// Load the new sample for this pixel we have just traced
				RGBA = NewRGBA;
				RGBA2 = NewRGBA2;
				DepthData = NewDepthData;
			}
			else if (bValidPreviousUVs)
			{
				float4 ReprojRGBA = SafeSamplePreviousFrameVolumetricTexture(PrevScreenUVs);
				float4 ReprojRGBA2 = SafeSamplePreviousFrameVolumetricSecondaryTexture(PrevScreenUVs);
				FDepthData ReprojDepthData = HistoryDepthData;

				// Min and Max the reprojected data to avoid reprojection issues. But this is too much constraint to use only the new sample as it causes jittering...
				//if(false)
				//{
				//	float4 MinRGBA = min(NewRGBA, NewRGBA2);
				//	float4 MaxRGBA = max(NewRGBA, NewRGBA2);
				//	ReprojRGBA = clamp(ReprojRGBA, MinRGBA, MaxRGBA);
				//	ReprojRGBA2 = clamp(ReprojRGBA2, MinRGBA, MaxRGBA);
				//}

				// Ignore reprojected samples with too large of a min depth difference that have been missed by dissoclusion.
				// Those samples are considered as an edge so we skip because it might reproject bad bilinearly filtered data over the background and use the new incoming data.
				// This is an attempt to fix missing cloud over sky background around close moving meshes. 
				// It should hopefully not disrupt forced bilinear filtering of background clouds (when depth is > MinimumDistanceKmToDisableDisoclusion).
				if (abs(NewDepthData.MinMaxViewDepthKm.x - HistoryDepthData.MinMaxViewDepthKm.x) > 1.0f)
				{
					ReprojRGBA = NewRGBA;// +float4(0.2, 0, 0, 0);
					ReprojRGBA2 = NewRGBA2;
					ReprojDepthData = NewDepthData;
				}

				RGBA = ReprojRGBA;
				RGBA2 = ReprojRGBA2;
				DepthData = ReprojDepthData;

				if (!(all(IsFinite(RGBA)) && all(IsFinite(RGBA2)) && all(IsFinite(float2(DepthData.CloudFrontDepthFromViewKm, DepthData.SceneDepthFromViewKm)))))
				{
					RGBA = float4(0.0f, 0.0f, 0.0f, 1.0f);
					RGBA2 = float4(0.0f, 0.0f, 0.0f, 1.0f);
					DepthData.CloudFrontDepthFromViewKm = 1000.0f;
					DepthData.SceneDepthFromViewKm = 1000.0f;
					DepthData.MinMaxViewDepthKm = 1000.0f.xx;
				}
			}
			else // !bValidPreviousUVs
			{
				// History is invalid so simply use this frame low resolution render with bilinear sampling.
				// Single sampel of the far data seem sto always be enough
				RGBA = SafeSampleTracingVolumetricTexture(ScreenUV);
				RGBA2 = SafeSampleSecondaryTracingVolumetricTexture(ScreenUV);
				DepthData = SafeSampleTracingVolumetricDepthTexture(ScreenUV);
			}

			//if (bApplyDisoclusion)
			//{
			//	RGBA *= float4(1, 1, 0, 1);
			//	RGBA2 *= float4(1, 1, 0, 1);
			//}

		#else // PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH

			if (bUseNewSample)
			{
				// Load the new sample for this pixel we have just traced
				RGBA = NewRGBA;
				DepthData = NewDepthData;
			}
			else if (bValidPreviousUVs)
			{
				//
				// NOTE: This path is use when mode0 is not used or when compute is not available (min max depth permutation are only generated for compute path)
				//
				
				// Sample valid on screen history
				float4 HistoryRGBA = SafeSamplePreviousFrameVolumetricTexture(PrevScreenUVs);
				FDepthData HistoryDepthData = SafeSamplePreviousFrameVolumetricDepthTexture(PrevScreenUVs);

				// Get information about neightboors
				int2 NeightboorsOffset[8] = { int2(1,0), int2(1,1), int2(0,1), int2(-1,1), int2(-1,0), int2(-1,-1), int2(0,-1), int2(1,-1)};

				const float ReconstructDepthZ = HalfResDepthTexture.Load(int3(SVPos.xy + ViewViewRectMin, 0)).r;
				const float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(float4(CenterSample, ReconstructDepthZ, 1.0));
				const float PixelDistanceFromViewKm = length(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin) * CENTIMETER_TO_KILOMETER;

				RGBA = HistoryRGBA;
				DepthData = HistoryDepthData;

				if (/*ReconstructDepthZ > 0.0001f &&*/ abs(PixelDistanceFromViewKm - DepthData.SceneDepthFromViewKm) > PixelDistanceFromViewKm * 0.1f)
				{
					// History has a too large depth difference at depth discontinuities, use the data with closest depth within the neightborhood
					float ClosestDepth = 99999999.0f;
					for (int i = 0; i < 8; ++i)
					{
						FDepthData NeighboorsDepthData = SafeLoadTracingVolumetricDepthTexture(CenterSample + NeightboorsOffset[i]);
						const float NeighboorsClosestDepth = abs(PixelDistanceFromViewKm - NeighboorsDepthData.SceneDepthFromViewKm);
						if (NeighboorsClosestDepth < ClosestDepth)
						{
							ClosestDepth = NeighboorsClosestDepth;
							float4 NeighboorsRGBA = SafeLoadTracingVolumetricTexture(CenterSample + NeightboorsOffset[i]);
							RGBA = NeighboorsRGBA;// +float4(0, 1, 0, 0);
							DepthData = NeighboorsDepthData;
						} 
					}
					// After more testing, the code below looked unecessary
					//if (abs(PixelDistanceFromViewKm - NewDepths.y) < ClosestDepth)
					//{
					//	RGBA = NewRGBA;
					//	Depths = NewDepths;
					//}
					//RGBA += float4(0, 0.5, 0, 0);
				}
				else // Because of the test on bUseNewSample above, we know here that we are only dealing with reprojected data  //if(ReconstructDepthZ < 0.000001f)
				{
					// TODO: To use this, we need to make sure we prioritise pixe lwith under represented depth.
				#if PERMUTATION_REPROJECTION_BOX_CONSTRAINT
					// Make sure that history stay in the neightborhood color/transmittance/depth box after reprojection
					float4 ColorAABBMin = 999999999.0f;
					float4 ColorAABBMax = 0.0f;
					float2 DepthsAABBMin = 999999999.0f;
					float2 DepthsAABBMax = 0.0f;
					bool bApply = true;
					for (int i = 0; i < 8; ++i)
					{
						float4 ColorData = SafeLoadTracingVolumetricTexture(CenterSample + NeightboorsOffset[i]);
						FDepthData NeighboorsDepthData = SafeLoadTracingVolumetricDepthTexture(CenterSample + NeightboorsOffset[i]);
						float2 NeighboorsDepthData2 = float2(NeighboorsDepthData.CloudFrontDepthFromViewKm, NeighboorsDepthData.SceneDepthFromViewKm);
						ColorAABBMin = min(ColorAABBMin, ColorData);
						ColorAABBMax = max(ColorAABBMax, ColorData);
						DepthsAABBMin = min(DepthsAABBMin, NeighboorsDepthData2);
						DepthsAABBMax = max(DepthsAABBMax, NeighboorsDepthData2);
						bApply = bApply && NeighboorsDepthData2.y > 1000.0f;
					}
					ColorAABBMin = min(ColorAABBMin, NewRGBA);
					ColorAABBMax = max(ColorAABBMax, NewRGBA);
					DepthsAABBMin = min(DepthsAABBMin, float2(NewDepthData.CloudFrontDepthFromViewKm, NewDepthData.SceneDepthFromViewKm));
					DepthsAABBMax = max(DepthsAABBMax, float2(NewDepthData.CloudFrontDepthFromViewKm, NewDepthData.SceneDepthFromViewKm));
					bApply = bApply && NewDepthData.SceneDepthFromViewKm > 1000.0f;
					//if (bApply)
					{
						RGBA = clamp(RGBA, ColorAABBMin, ColorAABBMax);

						float2 Depths = clamp(float2(DepthData.CloudFrontDepthFromViewKm, DepthData.SceneDepthFromViewKm), DepthsAABBMin, DepthsAABBMax);

						DepthData.CloudFrontDepthFromViewKm = Depths.x;
						DepthData.SceneDepthFromViewKm = Depths.y;
					}
					//RGBA += float4(0, 0.8, 0.8, 0);
				#endif
				}

				if (!(all(IsFinite(RGBA)) && all(IsFinite(float2(DepthData.CloudFrontDepthFromViewKm, DepthData.SceneDepthFromViewKm)))))
				{
					RGBA = float4(0.0f, 0.0f, 0.0f, 1.0f);
					DepthData.CloudFrontDepthFromViewKm = 1000.0f;
					DepthData.SceneDepthFromViewKm = 1000.0f;
				}
			}
			else // !bValidPreviousUVs
			{
				// History is invalid so simply use this frame low resolution render with bilinear sampling.
				// Single sampel of the far data seem sto always be enough
				RGBA = SafeSampleTracingVolumetricTexture(ScreenUV);
				DepthData = SafeSampleTracingVolumetricDepthTexture(ScreenUV);
			}

		#endif // PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH


		}
	}

	OutputRt0 = RGBA;
#if PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
	OutputRt1 = RGBA2;
	OutputRt2 = float4(DepthData.CloudFrontDepthFromViewKm, DepthData.SceneDepthFromViewKm, DepthData.MinMaxViewDepthKm);
#else // PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
	OutputRt1 = float4(
		DepthData.CloudFrontDepthFromViewKm, 
		DepthData.SceneDepthFromViewKm,
		0.0, 0.0);	// unused components
#endif // PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH

#else // PERMUTATION_HISTORY_AVAILABLE

	// Simple bilinear upsample
	OutputRt0 = SafeSampleTracingVolumetricTexture(ScreenUV);

#if PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
	OutputRt1 = SafeSampleSecondaryTracingVolumetricTexture(ScreenUV);

	FDepthData DepthData = SafeSampleTracingVolumetricDepthTexture(ScreenUV);
	OutputRt2 = float4(DepthData.CloudFrontDepthFromViewKm, DepthData.SceneDepthFromViewKm, DepthData.MinMaxViewDepthKm);
#else // PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH
	FDepthData DepthData = SafeSampleTracingVolumetricDepthTexture(ScreenUV);
	OutputRt1 = float4(
		DepthData.CloudFrontDepthFromViewKm,
		DepthData.SceneDepthFromViewKm,
		0.0, 0.0);	// unused components
#endif // PERMUTATION_CLOUD_MIN_AND_MAX_DEPTH

#endif // PERMUTATION_HISTORY_AVAILABLE


#if APPLY_FOG_LATE
	// Apply fog on the a side reconstructed texture if latefog is used for fog on translucent. Otherwise the regular reconstructed cloud texture will be used and this one do not have fog in it.
	float2 FullResViewSVPos = 0*SourceViewRectMin + ScreenUV * SourceViewRectSize;
	float2 BufferSVPos = FullResViewSVPos / 4.0;
	OutputCloudAndFog = ApplyFogToCloudSampleFromVPosAndDepth(FullResViewSVPos, BufferSVPos, DepthData.CloudFrontDepthFromViewKm, OutputRt0, 0*SourceViewRectMin);
#endif
}

#endif // SHADER_RECONSTRUCT_VOLUMETRICRT



#ifdef SHADER_COMPOSE_VOLUMETRICRT

#include "SceneTexturesCommon.ush"

SamplerState LinearTextureSampler;

Texture2D<float4> VolumetricTexture;
Texture2D<float4> VolumetricSecondaryTexture;
Texture2D<float4> VolumetricDepthTexture;
uint4 VolumetricTextureValidCoordRect;
float4 VolumetricTextureValidUvRect;
float2 ViewSeparateFogDepthOriginOffset;
uint ForwardShadingEnable;
uint OutputAlphaHoldout;
float VolumeTracingStartDistanceFromCamera;

#if PERMUTATION_RENDER_UNDERWATER_BUFFER || PERMUTATION_COMPOSE_WITH_WATER
Texture2D WaterLinearDepthTexture;
SamplerState WaterLinearDepthSampler;
float4 SceneWithoutSingleLayerWaterViewRect;
float2 FullResolutionToWaterBufferScale;
#endif

#if PERMUTATION_MSAA_SAMPLE_COUNT > 1
Texture2DMS<float, PERMUTATION_MSAA_SAMPLE_COUNT> MSAADepthTexture;
#endif

#if INSTANCED_STEREO
// When rendering instanced stereo side by side, we may use first view's texture for both views. This wraps the coords so the second view can use first view's texture.
// (Do we need a new shader permutation here? Also, should really use ViewRect for wrapping)
uint2 WrapCoordsForInstancedViews(uint2 Coord, uint4 ValidRect)
{
	return uint2(
		(Coord.x > ValidRect.z) ? (Coord.x - ValidRect.z) : Coord.x,
		Coord.y
		);
}
float2 WrapUVsForInstancedViews(float2 UV, float4 ValidRect)
{
	return float2(
		(UV.x > ValidRect.z) ? (UV.x - ValidRect.z) : UV.x,
		UV.y
		);
}
#else
uint2 WrapCoordsForInstancedViews(uint2 Coord, uint4 ValidRect)
{
	return Coord;
}
float2 WrapUVsForInstancedViews(float2 UV, float4 ValidRect)
{
	return UV;
}
#endif

float4 SafeLoadVolumetricTexture(uint2 Coord)
{
	const uint2 WrappedCoords = WrapCoordsForInstancedViews(Coord, VolumetricTextureValidCoordRect);
	return VolumetricTexture.Load(uint3(clamp(WrappedCoords, VolumetricTextureValidCoordRect.xy, VolumetricTextureValidCoordRect.zw), 0));
}
float4 SafeSampleVolumetricTexture(float2 UV)
{
	const float2 WrappedUV = WrapUVsForInstancedViews(UV, VolumetricTextureValidUvRect);
	return VolumetricTexture.SampleLevel(LinearTextureSampler, clamp(WrappedUV, VolumetricTextureValidUvRect.xy, VolumetricTextureValidUvRect.zw), 0);
}
float4 SafeLoadVolumetricSecondaryTexture(uint2 Coord)
{
	const uint2 WrappedCoords = WrapCoordsForInstancedViews(Coord, VolumetricTextureValidCoordRect);
	return VolumetricSecondaryTexture.Load(uint3(clamp(WrappedCoords, VolumetricTextureValidCoordRect.xy, VolumetricTextureValidCoordRect.zw), 0));
}
float4 SafeSampleVolumetricSecondaryTexture(float2 UV)
{
	const float2 WrappedUV = WrapUVsForInstancedViews(UV, VolumetricTextureValidUvRect);
	return VolumetricSecondaryTexture.SampleLevel(LinearTextureSampler, clamp(WrappedUV, VolumetricTextureValidUvRect.xy, VolumetricTextureValidUvRect.zw), 0);
}
float4 SafeLoadVolumetricDepthTexture(uint2 Coord)
{
	const uint2 WrappedCoords = WrapCoordsForInstancedViews(Coord, VolumetricTextureValidCoordRect);
	return VolumetricDepthTexture.Load(uint3(clamp(WrappedCoords, VolumetricTextureValidCoordRect.xy, VolumetricTextureValidCoordRect.zw), 0));
}
float4 SafeSampleVolumetricDepthTexture(float2 UV)
{
	const float2 WrappedUV = WrapUVsForInstancedViews(UV, VolumetricTextureValidUvRect);
	return VolumetricDepthTexture.SampleLevel(LinearTextureSampler, clamp(WrappedUV, VolumetricTextureValidUvRect.xy, VolumetricTextureValidUvRect.zw), 0);
}

float4 VolumetricTextureSizeAndInvSize;
float UvOffsetSampleAcceptanceWeight;
float MinimumDistanceKmToDisableDisoclusion;
float2 FullResolutionToVolumetricBufferResolutionScale;

float4 ApplyFogToCloudFromUVsAndDepth(float2 UV, float DepthKilometer, float4 CloudRGBT)
{
#if APPLY_FOG_LATE
	float2 SVPos = (UV / (FullResolutionToVolumetricBufferResolutionScale.x *View.BufferSizeAndInvSize.xy * VolumetricTextureSizeAndInvSize.zw)) * View.BufferSizeAndInvSize.xy + View.ViewRectMin.xy;
	CloudRGBT = ApplyFogToCloudSampleFromVPosAndDepth(SVPos, SVPos, DepthKilometer, CloudRGBT, View.ViewRectMin.xy);
#endif // APPLY_FOG_LATE

	return CloudRGBT;
}

float4 ApplyFogToCloudFromUVs(float2 UV, float4 CloudRGBT)
{
#if APPLY_FOG_LATE
	float2 SVPos = (UV / (FullResolutionToVolumetricBufferResolutionScale.x * View.BufferSizeAndInvSize.xy * VolumetricTextureSizeAndInvSize.zw)) * View.BufferSizeAndInvSize.xy + View.ViewRectMin.xy;
	float VolumeFrontDepthKilometer = SafeSampleVolumetricDepthTexture(UV).x;
	CloudRGBT = ApplyFogToCloudSampleFromVPosAndDepth(SVPos, SVPos, VolumeFrontDepthKilometer, CloudRGBT, View.ViewRectMin.xy);
#endif // APPLY_FOG_LATE

	return CloudRGBT;
}


void ComposeVolumetricRTOverScenePS(
	in float4 SVPos : SV_POSITION,
	out float4 OutputRt0 : SV_Target0
#if PERMUTATION_MSAA_SAMPLE_COUNT > 1
	, in uint SampleIndex : SV_SampleIndex
#endif
)
{
	float2 CurResPixelCoord = float2(SVPos.xy);
	float2 ScreenUV = CurResPixelCoord * View.BufferSizeAndInvSize.zw;

	// Each view has its own volumetric textures, and their contents are always aligned to the top-left even when in splitscreen
	// As such, ScreenUVNoOffset (and other UVs derived from it) should be used to sample from volumetric textures, and for the ApplyFogToCloud functions
	float2 ScreenUVNoOffset = (CurResPixelCoord - View.ViewRectMin.xy) * View.BufferSizeAndInvSize.zw;
	float2 VolumeUV = FullResolutionToVolumetricBufferResolutionScale.x * (ScreenUVNoOffset * View.BufferSizeAndInvSize.xy * VolumetricTextureSizeAndInvSize.zw);


	//Make the offset be independent of aspect ratio, resolution scale, downsampling
	const float2 FullResOffsetUVScale = float2(1.0f, View.BufferSizeAndInvSize.x * View.BufferSizeAndInvSize.w)	// Aspect ratio correction
		* View.BufferSizeAndInvSize.zw											// Pixel size
		* FullResolutionToVolumetricBufferResolutionScale.y;					// Volumetric buffer downsample factor

	float2 Offset0Sample = (float2(Rand3DPCG16(int3(CurResPixelCoord, View.StateFrameIndexMod8)).xy) * rcp(65536.0)) * 2.0f - 1.0f;
	float2 Offset1Sample = (float2(Rand3DPCG16(int3(CurResPixelCoord, View.StateFrameIndexMod8 + 8)).xy) * rcp(65536.0)) * 2.0f - 1.0f;
	float2 Offset2Sample = (float2(Rand3DPCG16(int3(CurResPixelCoord, View.StateFrameIndexMod8 + 16)).xy) * rcp(65536.0)) * 2.0f - 1.0f;
	float2 Offset3Sample = (float2(Rand3DPCG16(int3(CurResPixelCoord, View.StateFrameIndexMod8 + 32)).xy) * rcp(65536.0)) * 2.0f - 1.0f;
	Offset0Sample = normalize(Offset0Sample);
	Offset1Sample = normalize(Offset1Sample);
	Offset2Sample = normalize(Offset2Sample);
	Offset3Sample = normalize(Offset3Sample);

	const float UvOffsetScale = 1.0f;
	float2 Offset0 = Offset0Sample * FullResOffsetUVScale * UvOffsetScale;
	float2 Offset1 = Offset1Sample * FullResOffsetUVScale * UvOffsetScale;
	float2 Offset2 = Offset2Sample * FullResOffsetUVScale * UvOffsetScale;
	float2 Offset3 = Offset3Sample * FullResOffsetUVScale * UvOffsetScale;

	float2 VolumeUVOffset0 = VolumeUV + Offset0;
	float2 VolumeUVOffset1 = VolumeUV + Offset1;
	float2 VolumeUVOffset2 = VolumeUV + Offset2;
	float2 VolumeUVOffset3 = VolumeUV + Offset3;

#if PERMUTATION_UPSAMPLINGMODE==0
	// Single bilinear sample
//	OutputRt0 = SafeLoadVolumetricTexture(VolumeUV * VolumetricTextureSizeAndInvSize.xy);			// Closest
	OutputRt0 = SafeLoadVolumetricSecondaryTexture(VolumeUV * VolumetricTextureSizeAndInvSize.xy);	// Furthest
	float4 VolumeFrontDepth0 = SafeSampleVolumetricDepthTexture(VolumeUV);
	OutputRt0 = ApplyFogToCloudSampleFromVPosAndDepth(CurResPixelCoord, CurResPixelCoord, VolumeFrontDepth0.x, OutputRt0, View.ViewRectMin.xy);
	return;

#elif PERMUTATION_UPSAMPLINGMODE==1
	// Jitter the source sample to add high frequency that can be resolved by TAA - 4 samples

	float4 Data0 = SafeSampleVolumetricTexture(VolumeUVOffset0);
	Data0 = ApplyFogToCloudFromUVs(VolumeUVOffset0, Data0);

	float4 Data1 = SafeSampleVolumetricTexture(VolumeUVOffset1);
	Data1 = ApplyFogToCloudFromUVs(VolumeUVOffset1, Data1);

	float4 Data2 = SafeSampleVolumetricTexture(VolumeUVOffset2);
	Data2 = ApplyFogToCloudFromUVs(VolumeUVOffset2, Data2);

	float4 Data3 = SafeSampleVolumetricTexture(VolumeUVOffset3);
	Data3 = ApplyFogToCloudFromUVs(VolumeUVOffset3, Data3);

	OutputRt0 = 0.25 * (Data0 + Data1 + Data2 + Data3);
	return;

#elif (PERMUTATION_UPSAMPLINGMODE==4 || PERMUTATION_UPSAMPLINGMODE==3 || PERMUTATION_UPSAMPLINGMODE==2)

#if PERMUTATION_RENDER_UNDERWATER_BUFFER
	// Adapt the UV to the relative water buffer size
	float2 WaterVolumeUV = VolumeUV * FullResolutionToWaterBufferScale.y;
	// Offset the uv to the view buffer region and take into account dynamic resolution scaling.
	float2 WaterDepthScreenUV = SceneWithoutSingleLayerWaterViewRect.xy + WaterVolumeUV * (View.ViewSizeAndInvSize.xy * View.BufferSizeAndInvSize.zw);

	float PixelLinearDepth = ConvertFromDeviceZ(WaterLinearDepthTexture.SampleLevel(WaterLinearDepthSampler, WaterDepthScreenUV, 0).r);


	float3 TranslatedWorldPosition = SvPositionToTranslatedWorld(float4(SVPos.xy, 0.5, 1.0));
	TranslatedWorldPosition = normalize(TranslatedWorldPosition - PrimaryView.TranslatedWorldCameraOrigin) * PixelLinearDepth + PrimaryView.TranslatedWorldCameraOrigin;

	float4 ClipPosition = mul(float4(TranslatedWorldPosition, 1.0), PrimaryView.TranslatedWorldToClip);
	ClipPosition /= ClipPosition.w;
	float PixelDeviceZ = ClipPosition.z;
	PixelDeviceZ = max(0.000000000001, PixelDeviceZ);

	float3 ScreenTranslatedWorldPosition = SvPositionToTranslatedWorld(float4(SVPos.xy, PixelDeviceZ, 1.0));

#else

	float2 SVPosDepth = SVPos.xy + ViewSeparateFogDepthOriginOffset;
#if  PERMUTATION_MSAA_SAMPLE_COUNT > 1
	float PixelDeviceZ = MSAADepthTexture.Load(int2(SVPosDepth), SampleIndex).x;
#else
	float PixelDeviceZ = SceneTexturesStruct.SceneDepthTexture.Load(uint3(SVPosDepth, 0)).r;
#endif
	PixelDeviceZ = max(0.000000000001, PixelDeviceZ);
	float3 ScreenTranslatedWorldPosition = SvPositionToTranslatedWorld(float4(SVPosDepth, PixelDeviceZ, 1.0));
#endif 

	float PixelDistanceFromView = length(PrimaryView.TranslatedWorldCameraOrigin - ScreenTranslatedWorldPosition);
	float PixelDistanceFromViewKm = PixelDistanceFromView * CENTIMETER_TO_KILOMETER;

	if (PixelDistanceFromViewKm < VolumeTracingStartDistanceFromCamera)
	{
		// This pixel is closer to the start tracing distance so simple discard it.
		// This also allows to easily remove cloud leaking on forground meshes.
		clip(-1.0f);
		return;
	}


#if PERMUTATION_COMPOSE_WITH_WATER
	// Now check that we are compositing a pixel that is not "water" to avoid applying clouds twice (they are already composited with the behind water layer scene).
	// We also lack depth information behind the water surface now so the composition would be wrong anyway.
	float WaterTestPixelLinearDepth = ConvertFromDeviceZ(WaterLinearDepthTexture.SampleLevel(WaterLinearDepthSampler, ScreenUV, 0).r);
	if (WaterTestPixelLinearDepth > PixelDistanceFromView)
	{
		// This pixel contains water, so skip it because clouds have already been composited in the "behind water scene color".
		clip(-1.0f);
		return;
	}
#endif



#if PERMUTATION_UPSAMPLINGMODE==2
	// Single pixel, forced mode when source and target resolution are matching
	float4 VolumeRGBT = SafeSampleVolumetricTexture(VolumeUV);
	float VolumeFrontDepth = SafeSampleVolumetricDepthTexture(VolumeUV).r;

	if (PixelDistanceFromViewKm > VolumeFrontDepth)
	{
		OutputRt0 = ApplyFogToCloudFromUVsAndDepth(VolumeUV, VolumeFrontDepth.x, VolumeRGBT);
	}
	else
	{
		OutputRt0 = float4(0.0f, 0.0f, 0.0f, 1.0f);
		clip(-1.0f);
	}
#elif PERMUTATION_UPSAMPLINGMODE==3
	// Jitter the source sample to add high frequency that can be resolved by TAA - 4 samples + depth test with linear sampling

#if 1
	float4 VolumeRGBT0 = SafeSampleVolumetricTexture(VolumeUVOffset0);
	float4 VolumeFrontDepth0 = SafeSampleVolumetricDepthTexture(VolumeUVOffset0);
	float4 VolumeRGBT1 = SafeSampleVolumetricTexture(VolumeUVOffset1);
	float4 VolumeFrontDepth1 = SafeSampleVolumetricDepthTexture(VolumeUVOffset1);
	float4 VolumeRGBT2 = SafeSampleVolumetricTexture(VolumeUVOffset2);
	float4 VolumeFrontDepth2 = SafeSampleVolumetricDepthTexture(VolumeUVOffset2);
	float4 VolumeRGBT3 = SafeSampleVolumetricTexture(VolumeUVOffset3);
	float4 VolumeFrontDepth3 = SafeSampleVolumetricDepthTexture(VolumeUVOffset3);
#else
	float4 VolumeRGBT0 = SafeLoadVolumetricTexture(VolumeUVOffset0 * VolumetricTextureSizeAndInvSize.xy);
	float4 VolumeFrontDepth0 = SafeLoadVolumetricDepthTexture(VolumeUVOffset0 * VolumetricTextureSizeAndInvSize.xy);
	float4 VolumeRGBT1 = SafeLoadVolumetricTexture(VolumeUVOffset1 * VolumetricTextureSizeAndInvSize.xy);
	float4 VolumeFrontDepth1 = SafeLoadVolumetricDepthTexture(VolumeUVOffset1 * VolumetricTextureSizeAndInvSize.xy);
	float4 VolumeRGBT2 = SafeLoadVolumetricTexture(VolumeUVOffset2 * VolumetricTextureSizeAndInvSize.xy);
	float4 VolumeFrontDepth2 = SafeLoadVolumetricDepthTexture(VolumeUVOffset2 * VolumetricTextureSizeAndInvSize.xy);
	float4 VolumeRGBT3 = SafeLoadVolumetricTexture(VolumeUVOffset3 * VolumetricTextureSizeAndInvSize.xy);
	float4 VolumeFrontDepth3 = SafeLoadVolumetricDepthTexture(VolumeUVOffset3 * VolumetricTextureSizeAndInvSize.xy);
#endif


	float ValidSampleCount = 0.0f;
	float4 DataAcc = 0.0f;
#if 1
	if (PixelDistanceFromViewKm > VolumeFrontDepth0.x) { VolumeRGBT0 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset0, VolumeFrontDepth0.x, VolumeRGBT0); DataAcc += VolumeRGBT0; ValidSampleCount += 1.0f; }
	if (PixelDistanceFromViewKm > VolumeFrontDepth1.x) { VolumeRGBT1 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset1, VolumeFrontDepth1.x, VolumeRGBT1); DataAcc += VolumeRGBT1; ValidSampleCount += 1.0f; }
	if (PixelDistanceFromViewKm > VolumeFrontDepth2.x) { VolumeRGBT2 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset2, VolumeFrontDepth2.x, VolumeRGBT2); DataAcc += VolumeRGBT2; ValidSampleCount += 1.0f; }
	if (PixelDistanceFromViewKm > VolumeFrontDepth3.x) { VolumeRGBT3 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset3, VolumeFrontDepth3.x, VolumeRGBT3); DataAcc += VolumeRGBT3; ValidSampleCount += 1.0f; }
#else
	float ClostestDepth = 999999999.0f;
	float ThisDepth;
	PixelDistanceFromViewKm = min(PixelDistanceFromViewKm, max(max(VolumeFrontDepth0.y, VolumeFrontDepth1.y), max(VolumeFrontDepth2.y, VolumeFrontDepth3.y))); // clamp to the maximum of the read depth to avoid no depth matching
	ThisDepth = abs(VolumeFrontDepth0.y - PixelDistanceFromViewKm); if (ThisDepth < ClostestDepth) { VolumeRGBT0 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset0, VolumeFrontDepth0.x, VolumeRGBT0); DataAcc = VolumeRGBT0; ValidSampleCount = 1.0f; ClostestDepth = ThisDepth; }
	ThisDepth = abs(VolumeFrontDepth1.y - PixelDistanceFromViewKm); if (ThisDepth < ClostestDepth) { VolumeRGBT1 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset1, VolumeFrontDepth1.x, VolumeRGBT1); DataAcc = VolumeRGBT1; ValidSampleCount = 1.0f; ClostestDepth = ThisDepth; }
	ThisDepth = abs(VolumeFrontDepth2.y - PixelDistanceFromViewKm); if (ThisDepth < ClostestDepth) { VolumeRGBT2 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset2, VolumeFrontDepth2.x, VolumeRGBT2); DataAcc = VolumeRGBT2; ValidSampleCount = 1.0f; ClostestDepth = ThisDepth; }
	ThisDepth = abs(VolumeFrontDepth3.y - PixelDistanceFromViewKm); if (ThisDepth < ClostestDepth) { VolumeRGBT3 = ApplyFogToCloudFromUVsAndDepth(VolumeUVOffset3, VolumeFrontDepth3.x, VolumeRGBT3); DataAcc = VolumeRGBT3; ValidSampleCount = 1.0f; ClostestDepth = ThisDepth; }
#endif

	if (ValidSampleCount > 0.0f)
	{
		OutputRt0 = DataAcc / ValidSampleCount;
	}
	else
	{
		OutputRt0 = float4(0.0f, 0.0f, 0.0f, 1.0f);
		clip(-1.0f);
	}

#elif PERMUTATION_UPSAMPLINGMODE==4
	// Bilateral upsampling

	int2 PixelPos = SVPos.xy - View.ViewRectMin.xy + ViewSeparateFogDepthOriginOffset;
	int2 VolumeCoordUInt = PixelPos / int(FullResolutionToVolumetricBufferResolutionScale.y);
	int OffsetX = (VolumeCoordUInt.x * int(FullResolutionToVolumetricBufferResolutionScale.y)) == PixelPos.x ? -1 : 1;
	int OffsetY = (VolumeCoordUInt.y * int(FullResolutionToVolumetricBufferResolutionScale.y)) == PixelPos.y ? -1 : 1;

#if PERMUTATION_RENDER_UNDERWATER_BUFFER

	// Special spimple stochastic sampling when under water.
	OutputRt0 = float4(0, 0, 0, 1);
	{
		VolumeUVOffset0 = WaterVolumeUV + Offset0;
		VolumeUVOffset1 = WaterVolumeUV + Offset1;
		VolumeUVOffset2 = WaterVolumeUV + Offset2;
		VolumeUVOffset3 = WaterVolumeUV + Offset3;

		float4 VolumeRGBT0 = SafeLoadVolumetricTexture(VolumeUVOffset0 * VolumetricTextureSizeAndInvSize.xy);
		float4 VolumeFrontDepth0 = SafeLoadVolumetricDepthTexture(VolumeUVOffset0 * VolumetricTextureSizeAndInvSize.xy);
		float4 VolumeRGBT1 = SafeLoadVolumetricTexture(VolumeUVOffset1 * VolumetricTextureSizeAndInvSize.xy);
		float4 VolumeFrontDepth1 = SafeLoadVolumetricDepthTexture(VolumeUVOffset1 * VolumetricTextureSizeAndInvSize.xy);
		float4 VolumeRGBT2 = SafeLoadVolumetricTexture(VolumeUVOffset2 * VolumetricTextureSizeAndInvSize.xy);
		float4 VolumeFrontDepth2 = SafeLoadVolumetricDepthTexture(VolumeUVOffset2 * VolumetricTextureSizeAndInvSize.xy);
		float4 VolumeRGBT3 = SafeLoadVolumetricTexture(VolumeUVOffset3 * VolumetricTextureSizeAndInvSize.xy);
		float4 VolumeFrontDepth3 = SafeLoadVolumetricDepthTexture(VolumeUVOffset3 * VolumetricTextureSizeAndInvSize.xy);

		float ValidSampleCount = 0.0f;
		float4 DataAcc = 0.0f;
		float DepthForFog = 0.0f;
		const float CloudFrontDepthTinyOffset = 0.001;

		// We are testing if the depth buffer is further than the cloud front depth and that the cloud front depth is actually in front of traced depth.
		if (PixelDistanceFromViewKm > (VolumeFrontDepth0.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth0.x < (VolumeFrontDepth0.y)) { DepthForFog += VolumeFrontDepth0.x; DataAcc += VolumeRGBT0; ValidSampleCount += 1.0f; }
		if (PixelDistanceFromViewKm > (VolumeFrontDepth1.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth1.x < (VolumeFrontDepth1.y)) { DepthForFog += VolumeFrontDepth1.x; DataAcc += VolumeRGBT1; ValidSampleCount += 1.0f; }
		if (PixelDistanceFromViewKm > (VolumeFrontDepth2.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth2.x < (VolumeFrontDepth2.y)) { DepthForFog += VolumeFrontDepth2.x; DataAcc += VolumeRGBT2; ValidSampleCount += 1.0f; }
		if (PixelDistanceFromViewKm > (VolumeFrontDepth3.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth3.x < (VolumeFrontDepth3.y)) { DepthForFog += VolumeFrontDepth3.x; DataAcc += VolumeRGBT3; ValidSampleCount += 1.0f; }
		if (ValidSampleCount > 0.0f)
		{
		#if APPLY_FOG_LATE
			if (ValidSampleCount > 0.0f)
			{
				// Apply fog on a single sample cloud + depth result.
				DataAcc /= ValidSampleCount;
				DepthForFog /= ValidSampleCount;
				ValidSampleCount = 1.0;
				DataAcc = ApplyFogToCloudFromUVsAndDepth(WaterVolumeUV, DepthForFog, DataAcc);
			}
		#endif
			OutputRt0 = DataAcc / ValidSampleCount;
		}
		else
		{
			// If with the regular sampling we have not hit any valid data, let's sample further with an arbitrary scale.
			const float ArbitraryScale = 3.0f;
			VolumeUVOffset0 = WaterVolumeUV + Offset0.yx * ArbitraryScale;
			VolumeUVOffset1 = WaterVolumeUV + Offset1.yx * ArbitraryScale;
			VolumeUVOffset2 = WaterVolumeUV + Offset2.yx * ArbitraryScale;
			VolumeUVOffset3 = WaterVolumeUV + Offset3.yx * ArbitraryScale;

			VolumeRGBT0 = SafeLoadVolumetricTexture(VolumeUVOffset0 * VolumetricTextureSizeAndInvSize.xy);
			VolumeFrontDepth0 = SafeLoadVolumetricDepthTexture(VolumeUVOffset0 * VolumetricTextureSizeAndInvSize.xy);
			VolumeRGBT1 = SafeLoadVolumetricTexture(VolumeUVOffset1 * VolumetricTextureSizeAndInvSize.xy);
			VolumeFrontDepth1 = SafeLoadVolumetricDepthTexture(VolumeUVOffset1 * VolumetricTextureSizeAndInvSize.xy);
			VolumeRGBT2 = SafeLoadVolumetricTexture(VolumeUVOffset2 * VolumetricTextureSizeAndInvSize.xy);
			VolumeFrontDepth2 = SafeLoadVolumetricDepthTexture(VolumeUVOffset2 * VolumetricTextureSizeAndInvSize.xy);
			VolumeRGBT3 = SafeLoadVolumetricTexture(VolumeUVOffset3 * VolumetricTextureSizeAndInvSize.xy);
			VolumeFrontDepth3 = SafeLoadVolumetricDepthTexture(VolumeUVOffset3 * VolumetricTextureSizeAndInvSize.xy);

			// We are testing if the depth buffer is further than the cloud front depth and that the cloud front depth is actually in front of traced depth.
			if (PixelDistanceFromViewKm > (VolumeFrontDepth0.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth0.x < (VolumeFrontDepth0.y)) { DepthForFog += VolumeFrontDepth0.x; DataAcc += VolumeRGBT0; ValidSampleCount += 1.0f; }
			if (PixelDistanceFromViewKm > (VolumeFrontDepth1.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth1.x < (VolumeFrontDepth1.y)) { DepthForFog += VolumeFrontDepth1.x; DataAcc += VolumeRGBT1; ValidSampleCount += 1.0f; }
			if (PixelDistanceFromViewKm > (VolumeFrontDepth2.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth2.x < (VolumeFrontDepth2.y)) { DepthForFog += VolumeFrontDepth2.x; DataAcc += VolumeRGBT2; ValidSampleCount += 1.0f; }
			if (PixelDistanceFromViewKm > (VolumeFrontDepth3.x - CloudFrontDepthTinyOffset) && VolumeFrontDepth3.x < (VolumeFrontDepth3.y)) { DepthForFog += VolumeFrontDepth3.x; DataAcc += VolumeRGBT3; ValidSampleCount += 1.0f; }
			if (ValidSampleCount > 0.0f)
			{
			#if APPLY_FOG_LATE
				if (ValidSampleCount > 0.0f)
				{
					// Apply fog on a single sample cloud + depth result.
					DataAcc /= ValidSampleCount;
					DepthForFog /= ValidSampleCount;
					ValidSampleCount = 1.0;
					DataAcc = ApplyFogToCloudFromUVsAndDepth(WaterVolumeUV, DepthForFog, DataAcc);
				}
			#endif
				OutputRt0 = DataAcc / ValidSampleCount;
			}
			else
			{
				OutputRt0 = float4(0.0f, 0.0f, 0.0f, 1.0f);
				clip(-1.0f);
			}
		}
	}
	return;
#else // PERMUTATION_RENDER_UNDERWATER_BUFFER


	// We only want to run the special "cloud over water" code path on actual water pixels, otherwise depth threshold can cause visual issues.
	// However, with Forward Shading, we do not have access to the gbuffer: so we still run that simple water depth test to occlude cloud on forrground meshes. Not the perfect but it will work in most cases.
#if SUBSTRATE_GBUFFER_FORMAT==1
	FSubstrateAddressing SubstrateAddressing = GetSubstratePixelDataByteOffset(PixelPos, uint2(View.BufferSizeAndInvSize.xy), Substrate.MaxBytesPerPixel);
	FSubstratePixelHeader SubstratePixelHeader = UnpackSubstrateHeaderIn(Substrate.MaterialTextureArray, SubstrateAddressing, Substrate.TopLayerTexture);
	const bool bIsWaterPixel = SubstratePixelHeader.IsSingleLayerWater();
#else
	FScreenSpaceData ScreenSpaceData = GetScreenSpaceData(ScreenUV);
	const bool bIsWaterPixel = ForwardShadingEnable || ScreenSpaceData.GBuffer.ShadingModelID == SHADINGMODELID_SINGLELAYERWATER;
#endif

	uint2 VolumeCoord0 = max(0, int2(VolumeCoordUInt) + int2(0, 0));
	uint2 VolumeCoord1 = max(0, int2(VolumeCoordUInt) + int2(OffsetX, 0));
	uint2 VolumeCoord2 = max(0, int2(VolumeCoordUInt) + int2(OffsetX, OffsetY));
	uint2 VolumeCoord3 = max(0, int2(VolumeCoordUInt) + int2(0, OffsetY));

	float4 VolumeFrontDepth0 = SafeLoadVolumetricDepthTexture(VolumeCoord0);
	float4 VolumeFrontDepth1 = SafeLoadVolumetricDepthTexture(VolumeCoord1);
	float4 VolumeFrontDepth2 = SafeLoadVolumetricDepthTexture(VolumeCoord2);
	float4 VolumeFrontDepth3 = SafeLoadVolumetricDepthTexture(VolumeCoord3);



#if 0
//	OutputRt0 = SafeLoadVolumetricSecondaryTexture(VolumeCoord0);
	OutputRt0 = SafeLoadVolumetricTexture(VolumeUVOffset0 * VolumetricTextureSizeAndInvSize.xy);
	return;
#endif



#if PERMUTATION_MINMAXDEPTH_AVAILABLE == 0
	// ==> This path is when compute shader is disable and so no minmax depth texture is available. It is also used by VRT.Mode=1 for upsampling based on front depth only.
		
	// Check that cloud medium sample are in front of the surface we are upsampling over water to make sure we will have valid sample.
	// This is especially needed when upsampling cloud over water surface where depth is not matching the depth for which clouds have been traced for.
	// If samples are behind, we assume clouds should not be visible (it will be a harsh transition for now)
	const bool bAllCloudSamplesInFrontOfWater = all(float4(VolumeFrontDepth0.x, VolumeFrontDepth1.x, VolumeFrontDepth2.x, VolumeFrontDepth3.x) < PixelDistanceFromViewKm);
	const bool bAnyCloudSamplesInFrontOfWater = any(float4(VolumeFrontDepth0.x, VolumeFrontDepth1.x, VolumeFrontDepth2.x, VolumeFrontDepth3.x) < PixelDistanceFromViewKm);

	// clamp to the maximum of the read depth to avoid no depth matching
	PixelDistanceFromViewKm = min(PixelDistanceFromViewKm, max(max(VolumeFrontDepth0.y, VolumeFrontDepth1.y), max(VolumeFrontDepth2.y, VolumeFrontDepth3.y)));

	float4 Depth4 = float4(VolumeFrontDepth0.y, VolumeFrontDepth1.y, VolumeFrontDepth2.y, VolumeFrontDepth3.y);
	float4 Depth4Diff = PixelDistanceFromViewKm - Depth4;
	Depth4Diff = abs(Depth4Diff);
	float MaxDepth4Diff = max(max(Depth4Diff.x, Depth4Diff.y), max(Depth4Diff.z, Depth4Diff.w));

	float ValidSampleCount = 0;
	float4 DataAcc = 0;
	const float WeightMultiplier = 1000.0f;
	const float ThresholdToBilinear = PixelDistanceFromViewKm * 0.1;
	if (MaxDepth4Diff > ThresholdToBilinear)
	{
		float4 VolumeRGBT0 = SafeLoadVolumetricTexture(VolumeCoord0);
		float4 VolumeRGBT1 = SafeLoadVolumetricTexture(VolumeCoord1);
		float4 VolumeRGBT2 = SafeLoadVolumetricTexture(VolumeCoord2);
		float4 VolumeRGBT3 = SafeLoadVolumetricTexture(VolumeCoord3);

		float DepthForFog = 0.0f;

		if (bIsWaterPixel)
		{
			// Only do the following is the pixel is water
			const float ConsideredFarWaterDistanceKm = 1.0f;				// We only apply the fix for pixel that are further than 1km
			const float NotRenderedPixelDistanceKm = 500.0f;				// We assume pixel have a depth greater than that such distant depth pixel
			if (any(Depth4 > NotRenderedPixelDistanceKm)					// Some pixels have non rendered pixel...
				&& !all(Depth4 > NotRenderedPixelDistanceKm)				// ...but not all (in this case we want to fall back to bilinear filtering).
				&& PixelDistanceFromViewKm > ConsideredFarWaterDistanceKm)	// Only treat this way pixel that are far enough to not mix up close object with far water.
			{
				// This is a special case / hack added for to fix some visual issues encoutnered with water as of today.
				// Water is render after checker boarded minmax depth is taken for cloud rendering (because clouds also needs to be rendered within the water pass).
				// As such, the reconstructure fails a taking a matching color for a matching depth (cloud depth = 1000km but water is closer, less than 2 kilometers).
				// This does not handle all the cases but fixes current issues with the water system.
				// ==> this code can be removed when we automatically fix up edges of objects to have sharp silhouette under strong conflicting motion.
				const float AcceptanceOffsetKm = 0.001f;
				if (VolumeFrontDepth0.y > (PixelDistanceFromViewKm - AcceptanceOffsetKm))
				{
					DataAcc += VolumeRGBT0;
					DepthForFog += VolumeFrontDepth0.x;
					ValidSampleCount++;
				}
				if (VolumeFrontDepth1.y > (PixelDistanceFromViewKm - AcceptanceOffsetKm))
				{
					DataAcc += VolumeRGBT1;
					DepthForFog += VolumeFrontDepth1.x;
					ValidSampleCount++;
				}
				if (VolumeFrontDepth2.y > (PixelDistanceFromViewKm - AcceptanceOffsetKm))
				{
					DataAcc += VolumeRGBT2;
					DepthForFog += VolumeFrontDepth2.x;
					ValidSampleCount++;
				}
				if (VolumeFrontDepth3.y > (PixelDistanceFromViewKm - AcceptanceOffsetKm))
				{
					DataAcc += VolumeRGBT3;
					DepthForFog += VolumeFrontDepth3.x;
					ValidSampleCount++;
				}
			}
		}

		if (!bIsWaterPixel || (bIsWaterPixel && ValidSampleCount == 0 && bAnyCloudSamplesInFrontOfWater))
		{
			// Depth discontinuities edges
			float4 weights = 1.0f / (Depth4Diff * WeightMultiplier + 1.0f);
			const float weightsSum = dot(weights, float4(1.0f, 1.0f, 1.0f, 1.0f));
			weights /= weightsSum;

			ValidSampleCount = weightsSum > 0.0f ? 1.0 : 0.0f;
			DataAcc = weights.x * VolumeRGBT0 + weights.y * VolumeRGBT1 + weights.z * VolumeRGBT2 + weights.w * VolumeRGBT3;
			DepthForFog = weights.x * VolumeFrontDepth0.x + weights.y * VolumeFrontDepth1.x + weights.z * VolumeFrontDepth2.x + weights.w * VolumeFrontDepth3.x;
		}

	#if APPLY_FOG_LATE
		if (ValidSampleCount > 0.0f)
		{
			// Apply fog on a single sample cloud + depth result.
			DataAcc /= ValidSampleCount;
			DepthForFog /= ValidSampleCount;
			ValidSampleCount = 1.0;
			DataAcc = ApplyFogToCloudFromUVsAndDepth(ScreenUVNoOffset, DepthForFog, DataAcc);
		}
	#endif
	}
	else
	{
		// Now do a bilinear sample to have a soft look region without depth edges.
		ValidSampleCount = 1.0;
		float2 SampleUVs = (float2(VolumeCoord0) + 0.25 + (float2(OffsetX, OffsetY) * 0.5 + 0.5) * 0.5) * VolumetricTextureSizeAndInvSize.zw;
		DataAcc = SafeSampleVolumetricTexture(SampleUVs);
		DataAcc = ApplyFogToCloudFromUVs(SampleUVs, DataAcc);
	}

	OutputRt0 = float4(0.0f, 0.0f, 0.0f, 1.0f);
	if (ValidSampleCount > 0.0f)
	{
		OutputRt0 = DataAcc / ValidSampleCount;
	}
	else
	{
		clip(-1.0f);
	}
	return;

#else // PERMUTATION_MINMAXDEPTH_AVAILABLE == 1

	// ==> Now this is the higher quality upsampling using minmax depth/luminance/trabnsmittance per pixel from reconstruction.

	if (bIsWaterPixel)
	{
		// Clamp far depth to water depth so that upsampling works and select appropriate surface (if depth=inf, the cloud on such depth composited by water later on will never be selected)
		VolumeFrontDepth0.w = min(VolumeFrontDepth0.w, PixelDistanceFromViewKm);
		VolumeFrontDepth1.w = min(VolumeFrontDepth1.w, PixelDistanceFromViewKm);
		VolumeFrontDepth2.w = min(VolumeFrontDepth2.w, PixelDistanceFromViewKm);
		VolumeFrontDepth3.w = min(VolumeFrontDepth3.w, PixelDistanceFromViewKm);
	}
	//PixelDistanceFromViewKm = min(PixelDistanceFromViewKm, max(max(VolumeFrontDepth0.w, VolumeFrontDepth1.w), max(VolumeFrontDepth2.w, VolumeFrontDepth3.w)));

	// Check that cloud medium sample are in front of the surface we are upsampling over water to make sure we will have valid sample.
	// This is especially needed when upsampling cloud over water surface where depth is not matching the depth for which clouds have been traced for.
	// If samples are all in front, we can simply upsample using the further traced clouds when for water pixel.
//	const bool bAllCloudSamplesInFrontOfWater = bIsWaterPixel && all(float4(VolumeFrontDepth0.x, VolumeFrontDepth1.x, VolumeFrontDepth2.x, VolumeFrontDepth3.x) < PixelDistanceFromViewKm);
	const bool bAllCloudSamplesInFrontOfDepth = all(float4(VolumeFrontDepth0.x, VolumeFrontDepth1.x, VolumeFrontDepth2.x, VolumeFrontDepth3.x) < (PixelDistanceFromViewKm-0.01));
	const bool bAnyCloudSamplesInFrontOfWater = any(float4(VolumeFrontDepth0.x, VolumeFrontDepth1.x, VolumeFrontDepth2.x, VolumeFrontDepth3.x) < PixelDistanceFromViewKm);

	float ValidSampleCount = 0;
	float4 DataAcc = 0;
	if (!bIsWaterPixel || (bIsWaterPixel && ValidSampleCount == 0 && bAnyCloudSamplesInFrontOfWater))
	{
		float4 VolumeRGBT0, VolumeRGBT1, VolumeRGBT2, VolumeRGBT3;
		float4 Depth4;


		float MinMaxDepthRange = 0;

		uint2 VolumeCoord;
		float4 VolumeFrontDepth;
		float DepthDiff;
		bool bSampleFront;
		float Tmp;

		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth0.w);
		//if (Tmp < DepthDiff)	// Accept the first fragment
		{
			VolumeCoord = VolumeCoord0;
			VolumeFrontDepth = VolumeFrontDepth0.w;
			DepthDiff = Tmp;
			bSampleFront = false;
		}
		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth0.z);
		if (Tmp < DepthDiff)
		{
			VolumeCoord = VolumeCoord0;
			VolumeFrontDepth = VolumeFrontDepth0.z;
			DepthDiff = Tmp;
			bSampleFront = true;
		}

		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth1.w);
		if (Tmp < DepthDiff)
		{
			VolumeCoord = VolumeCoord1;
			VolumeFrontDepth = VolumeFrontDepth1.w;
			DepthDiff = Tmp;
			bSampleFront = false;
		}
		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth1.z);
		if (Tmp < DepthDiff)
		{
			VolumeCoord = VolumeCoord1;
			VolumeFrontDepth = VolumeFrontDepth1.z;
			DepthDiff = Tmp;
			bSampleFront = true;
		}

		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth2.w);
		if (Tmp < DepthDiff)
		{
			VolumeCoord = VolumeCoord2;
			VolumeFrontDepth = VolumeFrontDepth2.w;
			DepthDiff = Tmp;
			bSampleFront = false;
		}
		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth2.z);
		if (Tmp < DepthDiff)
		{
			VolumeCoord = VolumeCoord2;
			VolumeFrontDepth = VolumeFrontDepth2.z;
			DepthDiff = Tmp;
			bSampleFront = true;
		}

		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth3.w);
		if (Tmp < DepthDiff)
		{
			VolumeCoord = VolumeCoord3;
			VolumeFrontDepth = VolumeFrontDepth3.w;
			DepthDiff = Tmp;
			bSampleFront = false;
		}
		Tmp = abs(PixelDistanceFromViewKm - VolumeFrontDepth3.z);
		if (Tmp < DepthDiff)
		{
			VolumeCoord = VolumeCoord3;
			VolumeFrontDepth = VolumeFrontDepth3.z;
			DepthDiff = Tmp;
			bSampleFront = true;
		}

		float MaxDepth = max(max(max(VolumeFrontDepth0.w, VolumeFrontDepth0.z), max(VolumeFrontDepth1.w, VolumeFrontDepth1.z)), max(max(VolumeFrontDepth2.w, VolumeFrontDepth2.z), max(VolumeFrontDepth3.w, VolumeFrontDepth3.z)));
		float MinDepth = min(min(min(VolumeFrontDepth0.w, VolumeFrontDepth0.z), min(VolumeFrontDepth1.w, VolumeFrontDepth1.z)), min(min(VolumeFrontDepth2.w, VolumeFrontDepth2.z), min(VolumeFrontDepth3.w, VolumeFrontDepth3.z)));
		MinMaxDepthRange = MaxDepth - MinDepth;

#define DEBUG_UPSAMPLING 0

		const bool bAllCloudSamplesBehindFrontDepth = all(float4(VolumeFrontDepth0.x, VolumeFrontDepth1.x, VolumeFrontDepth2.x, VolumeFrontDepth3.x) > (PixelDistanceFromViewKm + 0.001));
		if (bAllCloudSamplesBehindFrontDepth)
		{
		#if DEBUG_UPSAMPLING
			OutputRt0 = float4(0.0f, 1.0f, 0.0f, 0.5f);
		#else
			clip(-1.0f);
		#endif
			return;
		}


		// TODO parameterise from cvar
		const float DepthDifferenceToBilinearKm	= 1.0;
		if (!bIsWaterPixel)
		{
			// All cloud samples are in front of the min depth buffer
			const bool bAllCloudInFrontOfMinDepth = all(float4(VolumeFrontDepth0.x, VolumeFrontDepth1.x, VolumeFrontDepth2.x, VolumeFrontDepth3.x) <= float4(VolumeFrontDepth0.z, VolumeFrontDepth1.z, VolumeFrontDepth2.z, VolumeFrontDepth3.z));

			const bool bAllCloudToBilinear = 
						bAllCloudInFrontOfMinDepth
				// All depth values are further than thte distance at which we disable upsampling.
					&& (all(VolumeFrontDepth0.yzw >= MinimumDistanceKmToDisableDisoclusion) && all(VolumeFrontDepth1.yzw >= MinimumDistanceKmToDisableDisoclusion) 
						&& all(VolumeFrontDepth2.yzw >= MinimumDistanceKmToDisableDisoclusion) && all(VolumeFrontDepth3.yzw >= MinimumDistanceKmToDisableDisoclusion));
			// => Note: We could not enable bilinear sampling when all front cloud depths are in front of scene depth ONLY, 
			//			This is because if a mountain is visible through and intersecting a cloud not too far from the camera, low resolution texels might become visible.
			//			So unfortunately this does not work for an isolated cloud in front of a mountain.
			//			That is why we use the && operator. 
			// => The case of mountain visible inside a cloud is handled by the other test below. 

			if ((bAllCloudInFrontOfMinDepth && (MinMaxDepthRange < DepthDifferenceToBilinearKm)) || bAllCloudToBilinear)
			{
				// Bilinear upsampling for a soft look.
				float2 SampleUVs = (float2(VolumeCoord0) + 0.25 + (float2(OffsetX, OffsetY) * 0.5 + 0.5) * 0.5) * VolumetricTextureSizeAndInvSize.zw;
				DataAcc = SafeSampleVolumetricTexture(SampleUVs);

				ValidSampleCount = 1.0;
				DataAcc = ApplyFogToCloudFromUVs(SampleUVs, DataAcc);

				#if DEBUG_UPSAMPLING
				DataAcc += float4(0.0, 0.0, 0.1, 0.0);
				#endif
			}
			else
			{
				// Choose the best fit for the 2x2 neightborhood around edges
				float FogDepth = 0;
				BRANCH
				if (bSampleFront)
				{
					DataAcc = SafeLoadVolumetricSecondaryTexture(VolumeCoord);
					FogDepth = VolumeFrontDepth.z;
				}
				else
				{
					DataAcc = SafeLoadVolumetricTexture(VolumeCoord);
					FogDepth = VolumeFrontDepth.w; 
				}

				ValidSampleCount = 1;
				DataAcc = ApplyFogToCloudFromUVsAndDepth(VolumeCoord * VolumetricTextureSizeAndInvSize.zw, FogDepth, DataAcc);

				#if DEBUG_UPSAMPLING
				DataAcc += bSampleFront ? float4(0.1, 0.0, 0.0, 0.0) : float4(0.0, 0.1, 0.0, 0.0);
				#endif
			}
		}
		else
		{
			// Bilinear upsampling for a soft look.
			ValidSampleCount = 1.0;
			float2 SampleUVs = (float2(VolumeCoord0) + 0.25 + (float2(OffsetX, OffsetY) * 0.5 + 0.5) * 0.5) * VolumetricTextureSizeAndInvSize.zw;
			DataAcc = SafeSampleVolumetricTexture(SampleUVs);
			DataAcc = ApplyFogToCloudFromUVs(SampleUVs, DataAcc);

			#if DEBUG_UPSAMPLING
			DataAcc += float4(0.1, 0.0, 0.1, 0.0);
			#endif
		}
	}


	OutputRt0 = float4(0.0f, 0.0f, 0.0f, 1.0f);
	if (ValidSampleCount > 0.0f)
	{
		OutputRt0 = DataAcc / ValidSampleCount;
	}
	else
	{
		clip(-1.0f);
	}

#endif // PERMUTATION_MINMAXDEPTH_AVAILABLE

#endif // PERMUTATION_RENDER_UNDERWATER_BUFFER

#endif

#endif // PERMUTATION_UPSAMPLINGMODE==4 || PERMUTATION_UPSAMPLINGMODE==3 || PERMUTATION_UPSAMPLINGMODE==2

#if SUPPORT_PRIMITIVE_ALPHA_HOLDOUT
	if (OutputAlphaHoldout)
	{
		OutputRt0 = float4(0.0, 0.0, 0.0, OutputRt0.r);
	}
#endif
}

#endif // SHADER_COMPOSE_VOLUMETRICRT



