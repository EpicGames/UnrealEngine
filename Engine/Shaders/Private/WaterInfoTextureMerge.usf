// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

Texture2D<float> GroundDepthTexture;
SamplerState GroundDepthTextureSampler;
Texture2D<float4> WaterBodyTexture;
SamplerState WaterBodyTextureSampler;
Texture2D<float> WaterBodyDepthTexture;
SamplerState WaterBodyDepthTextureSampler;
Texture2D<float> DilatedWaterBodyDepthTexture;
SamplerState DilatedWaterBodyDepthTextureSampler;

float2 WaterHeightExtents;
float GroundZMin;
float CaptureZ;
float UndergroundDilationDepthOffset;
float DilationOverwriteMinimumDistance;

float NormalizeHeight(float Height, float Min, float Max)
{
	float NormalizedHeight = 1.f;
	// It's possible that the min and max are the same value in which case we might end up generating a div by zero error
	// In this case it's okay to use the Normalized value of 1.0 which will just map to the Min value.
	if (Min != Max)
	{
		NormalizedHeight = saturate((Height - Min) / (Max - Min));
	}
	return NormalizedHeight;
}

void Main(
	in float4 Position : SV_Position,
	out float4 OutColor : SV_Target0
)
{
	const float2 ScreenPosition = SvPositionToScreenPosition(Position).xy;
	float2 ScreenUV = ScreenPosition * View.ScreenPositionScaleBias.xy + View.ScreenPositionScaleBias.wz;

	// All input textures MUST be sampled with a point sampler to avoid a driver issue on older Android devices which occurs when using Load instead.
	const float2 WaterBodyVelocity = WaterBodyTexture.SampleLevel(WaterBodyTextureSampler, ScreenUV, 0).xy;
	const float WaterBodyDeviceDepth =  WaterBodyDepthTexture.SampleLevel(WaterBodyDepthTextureSampler, ScreenUV, 0);
	const float DilationDeviceDepth = DilatedWaterBodyDepthTexture.SampleLevel(DilatedWaterBodyDepthTextureSampler, ScreenUV, 0);
	const float GroundDeviceDepth =  GroundDepthTexture.SampleLevel(GroundDepthTextureSampler, ScreenUV, 0);

	const float GroundSceneDepth = ConvertFromDeviceZ(GroundDeviceDepth);
	float GroundZ = CaptureZ - GroundSceneDepth;

	const float WaterSceneDepth = ConvertFromDeviceZ(WaterBodyDeviceDepth);
	const float WaterZ = CaptureZ - WaterSceneDepth;

	const float DilationSceneDepth = ConvertFromDeviceZ(DilationDeviceDepth);
	const bool bPixelHasDilationData = DilationSceneDepth != 0.0;

	float DilatedZ = CaptureZ - DilationSceneDepth;


	// Always initialize output to avoid bugs in old shader compilers 
	OutColor = 0.f;

	// If the GroundDepth is further than the far plane of the projection, set the GroundZ to some really far away value
	if (GroundDeviceDepth == 1.0)
	{
		GroundZ = GroundZMin;
	}

	const float WaterZMin = WaterHeightExtents.x;
	const float WaterZMax = WaterHeightExtents.y;

	// if the measured WaterZ is below WaterZMin it means that there is no water at this pixel because it is not possible
	// for water to be further than ZMin.
	bool bPixelHasValidWaterData = WaterZ >= WaterZMin;

	// Additionally, if we have dilation data here and the water data would be placed below the landscape, instead draw the dilation data
	if (bPixelHasDilationData && (WaterZ + 128.f < GroundZ))
	{
		bPixelHasValidWaterData = false;
	}

	if (bPixelHasValidWaterData)
	{
		const float NormalizedWaterZ = NormalizeHeight(WaterZ, WaterZMin, WaterZMax);
		const float NormalizedGroundZ = NormalizeHeight(GroundZ, GroundZMin, WaterZMax);

		OutColor = float4(WaterBodyVelocity, NormalizedWaterZ, NormalizedGroundZ);
	}
	else if (bPixelHasDilationData)
	{

		// If the dilated water would render on top of terrain, squash it down below
		if (DilatedZ > GroundZ)
		{
			DilatedZ = GroundZ - UndergroundDilationDepthOffset;
		}

		const float NormalizedWaterZ = NormalizeHeight(DilatedZ, WaterZMin, WaterZMax);
		const float NormalizedGroundZ = NormalizeHeight(GroundZ, GroundZMin, WaterZMax);

		OutColor = float4(0.0, 0.0, NormalizedWaterZ, NormalizedGroundZ);
	}
	else
	{
		// Pixel may not have any dilation data nor water data. In this case just write 0s to the texture

		const float NormalizedGroundZ = NormalizeHeight(GroundZ, GroundZMin, WaterZMax);
		OutColor = float4(0.0, 0.0, 0.0, NormalizedGroundZ);
	}
}
