// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

uint CountBits( uint2 Bits )
{
	return countbits( Bits.x ) + countbits( Bits.y );
}

/*
===============================================================================
Below are a series of WaveInterlockedAdd (and WaveInterlockedMin/Max) macros used for aggregated atomics.

There is a varying and scalar version of each type, scalar being faster. Scalar refers to Value. I can't check if the scalar version
is actually called with a scalar since HLSL doesn't allow that so be careful.

These are all forms of aggregated atomics which replace an extremely common pattern in compute when appending to a buffer. 

To avoid lots of global atomic traffic you typically follow this pattern:
* if( GroupIndex == 0 ) set groupshared counter to zero
* Group sync outside of flow control
* Accumulate locally using InterlockedAdd to groupshared
* Get the local offset amongst the group backGroup sync outside of flow control
* if( GroupIndex == 0 ) do a single global InterlockedAdd to the buffer
* Get the global offset bac
* Add local offset to global offset to derive the actual index of the element you appended
* Finally write the element value you wanted to append to the buffer at that index

This is a pain, forces any such operation into at least 4 code blocks, and generally makes a mess of things, especially when you
have many different counters in the same shader. It also adds syncs and LDS traffic which might slow things down.

All this is unnecessary as it can be done per wave instead. Some PC drivers automatically convert InterlockedAdd's to scalar addresses
to this for you. Console compilers definitely do not. The above macros do this for you. They can be used whether waveops are supported
or not and simply fallback to slower non-aggregated atomics when they aren't. Drivers may then do the optimization for you if they do that.
Using these macros which use wave ops is almost certainly faster. Using them is definitely cleaner and easier.

The InGroups versions implement another common pattern which is needing to set indirect args in increments of work groups, not threads.

Unfortunately I had to use macros as we don't have templates and I have no idea anyways how to pass a reference to an element of a RWBuffer
that's to be written to as a parameter on all platforms. Also unfortunate is that you can't overload a macro, hence the _ names.
===============================================================================
*/

#if COMPILER_SUPPORTS_WAVE_VOTE

#define WaveInterlockedAddScalar( Dest, Value ) \
{ \
	uint NumToAdd = WaveActiveCountBits( true ) * Value; \
	if( WaveIsFirstLane() ) \
		InterlockedAdd( Dest, NumToAdd ); \
}

#define WaveInterlockedAddScalar_( Dest, Value, OriginalValue ) \
{ \
	uint NumToAdd = WaveActiveCountBits( true ) * Value; \
	OriginalValue = 0; \
	if( WaveIsFirstLane() ) \
		InterlockedAdd( Dest, NumToAdd, OriginalValue ); \
	OriginalValue = WaveReadLaneFirst( OriginalValue ) + WavePrefixCountBits( true ) * Value; \
}

#define WaveInterlockedAdd( Dest, Value ) \
{ \
	uint NumToAdd = WaveActiveSum( Value ); \
	if( WaveIsFirstLane() ) \
		InterlockedAdd( Dest, NumToAdd ); \
}

#define WaveInterlockedAdd_( Dest, Value, OriginalValue ) \
{ \
	uint LocalOffset = WavePrefixSum( Value ); \
	uint NumToAdd = WaveReadLaneLast( LocalOffset + Value ); \
	OriginalValue = 0; \
	if( WaveIsFirstLane() ) \
		InterlockedAdd( Dest, NumToAdd, OriginalValue ); \
	OriginalValue = WaveReadLaneFirst( OriginalValue ) + LocalOffset; \
}

#define WaveInterlockedAddScalarInGroups( Dest, DestGroups, GroupsOf, Value, OriginalValue ) \
{ \
	uint NumToAdd = WaveActiveCountBits( true ) * Value; \
	OriginalValue = 0; \
	if( WaveIsFirstLane() ) \
	{ \
		InterlockedAdd( Dest, NumToAdd, OriginalValue ); \
		InterlockedMax( DestGroups, ( OriginalValue + NumToAdd + GroupsOf - 1 ) / GroupsOf ); \
	} \
	OriginalValue = WaveReadLaneFirst( OriginalValue ) + WavePrefixCountBits( true ) * Value; \
}

#define WaveInterlockedAddInGroups( Dest, DestGroups, GroupsOf, Value, OriginalValue ) \
{ \
	uint LocalOffset = WavePrefixSum( Value ); \
	uint NumToAdd = WaveReadLaneLast( LocalOffset + Value ); \
	OriginalValue = 0; \
	if( WaveIsFirstLane() ) \
	{ \
		InterlockedAdd( Dest, NumToAdd, OriginalValue ); \
		InterlockedMax( DestGroups, ( OriginalValue + NumToAdd + GroupsOf - 1 ) / GroupsOf ); \
	} \
	OriginalValue = WaveReadLaneFirst( OriginalValue ) + LocalOffset; \
}

#define WaveInterlockedMin( Dest, Value ) \
{ \
	uint MinValue = WaveActiveMin( Value ); \
	if( WaveIsFirstLane() ) \
		InterlockedMin( Dest, MinValue ); \
}

#define WaveInterlockedMax( Dest, Value ) \
{ \
	uint MaxValue = WaveActiveMax( Value ); \
	if( WaveIsFirstLane() ) \
		InterlockedMax( Dest, MaxValue ); \
}

#define WaveInterlockedOr( Dest, Value ) \
{ \
	uint OrValue = WaveActiveBitOr( Value ); \
	if( WaveIsFirstLane() ) \
		InterlockedOr( Dest, OrValue ); \
}

#else

#define WaveInterlockedAddScalar( Dest, Value )					InterlockedAdd( Dest, Value )
#define WaveInterlockedAddScalar_( Dest, Value, OriginalValue )	InterlockedAdd( Dest, Value, OriginalValue )
#define WaveInterlockedAdd( Dest, Value )						InterlockedAdd( Dest, Value )
#define WaveInterlockedAdd_( Dest, Value, OriginalValue )		InterlockedAdd( Dest, Value, OriginalValue )

#define WaveInterlockedAddScalarInGroups( Dest, DestGroups, GroupsOf, Value, OriginalValue ) \
{ \
	InterlockedAdd( Dest, Value, OriginalValue ); \
	InterlockedMax( DestGroups, ( OriginalValue + Value + GroupsOf - 1 ) / GroupsOf ); \
}

#define WaveInterlockedAddInGroups( Dest, DestGroups, GroupsOf, Value, OriginalValue ) \
{ \
	InterlockedAdd( Dest, Value, OriginalValue ); \
	InterlockedMax( DestGroups, ( OriginalValue + Value + GroupsOf - 1 ) / GroupsOf ); \
}

#define WaveInterlockedMin( Dest, Value ) { InterlockedMin(Dest, Value); }
#define WaveInterlockedMax( Dest, Value ) { InterlockedMax(Dest, Value); }

#define WaveInterlockedOr( Dest, Value ) { InterlockedOr(Dest, Value); }


#endif // COMPILER_SUPPORTS_WAVE_VOTE


#if FEATURE_LEVEL >= FEATURE_LEVEL_SM6 || PLATFORM_SUPPORTS_SM6_0_WAVE_OPERATIONS

bool WaveReadLaneAt(bool In, uint SrcIndex)
{
	return (bool)WaveReadLaneAt((uint)In, SrcIndex);
}

// TODO: Workaround for WaveReadLaneAt being stripped for matrix types when compiling metal shaders
// Will remove when we switch to MSC
// Update: WaveReadLaneAt matrix overloads also cause pipelines to fail creation on Vulkan/NVIDIA
float4x4 WaveReadLaneAtMatrix(float4x4 In, uint SrcIndex)
{
	float4x4 Result;
	Result[0] = WaveReadLaneAt(In[0], SrcIndex);
	Result[1] = WaveReadLaneAt(In[1], SrcIndex);
	Result[2] = WaveReadLaneAt(In[2], SrcIndex);
	Result[3] = WaveReadLaneAt(In[3], SrcIndex);
	return Result;
}
float3x3 WaveReadLaneAtMatrix(float3x3 In, uint SrcIndex)
{
	float3x3 Result;
	Result[0] = WaveReadLaneAt(In[0], SrcIndex);
	Result[1] = WaveReadLaneAt(In[1], SrcIndex);
	Result[2] = WaveReadLaneAt(In[2], SrcIndex);
	return Result;
}

#if 0 // These are currently unsafe to use on Metal and Vulkan/NVIDIA. Don't reenable before those issues are fixed.
float4x4 WaveReadLaneAt(float4x4 In, uint SrcIndex)
{
	float4x4 Result;
	Result[0] = WaveReadLaneAt(In[0], SrcIndex);
	Result[1] = WaveReadLaneAt(In[1], SrcIndex);
	Result[2] = WaveReadLaneAt(In[2], SrcIndex);
	Result[3] = WaveReadLaneAt(In[3], SrcIndex);
	return Result;
}
float3x3 WaveReadLaneAt(float3x3 In, uint SrcIndex)
{
	float3x3 Result;
	Result[0] = WaveReadLaneAt(In[0], SrcIndex);
	Result[1] = WaveReadLaneAt(In[1], SrcIndex);
	Result[2] = WaveReadLaneAt(In[2], SrcIndex);
	return Result;
}
#endif

#endif