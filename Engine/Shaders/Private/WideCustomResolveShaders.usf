// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	WideCustomResolveShaders.usf: Custom wider filter resolve shaders
  =============================================================================*/

#include "Common.ush"

#ifdef OVERRIDE_WIDECUSTOMRESOLVESHADERS_USH
# include "/Platform/Private/WideCustomResolveShaders.ush"
#else

float2 ResolveOrigin;

float4 WideCustomResolveVS(uint Id : SV_VertexID) : SV_POSITION
{
	int x = Id &  1;
	int y = Id >> 1;

	return float4(x * 4 - 1, y * 4 - 1, 0, 1);
}

float ModifySampleWeight(float3 Sample)
{
	// Inverse luminance filtering
	return rcp(1.0 + Luminance(Sample));
}

float CalcSampleWeight(float w, float3 Sample)
{
	// Inverse luminance filtering
	float lum = Luminance(Sample)/w;
	return rcp(1.0/w + lum);
}

#if USE_FMASK
//FMASK not implemented for this platform.
#endif

#if defined(MSAA_SAMPLE_COUNT)
# if MSAA_SAMPLE_COUNT == 0

Texture2D<float4> Tex;

float4 WideCustomResolvePS(float4 Pos : SV_POSITION) : SV_Target0
{
	uint2 P = uint2(ResolveOrigin + Pos.xy);
	return float4(Tex.Load(int3(P, 0)).rgb, 0);
}

# else

void LoadTexSampleAndWeight(Texture2DMS<float4, MSAA_SAMPLE_COUNT> TargetTex, uint2 position, int sampleIndex, float weight, inout float3 sampleSum, inout float weightSum)
{
	float3 texSample = TargetTex.Load((int2)position, sampleIndex).xyz;
	float sampleW = CalcSampleWeight(weight, texSample);
	sampleSum += texSample*sampleW;
	weightSum += sampleW;
}

Texture2DMS<float4,MSAA_SAMPLE_COUNT> Tex; // Input MSAA color

# if WIDE_RESOLVE_WIDTH == 0
// Box filter
float3 resolve_bspline(uint2 pos)
{
	return (Tex.Load(pos, 0) + Tex.Load(pos, 1) + Tex.Load(pos, 2) + Tex.Load(pos, 3)).xyz * .25;
}
# elif WIDE_RESOLVE_WIDTH == 1
#  include "WideCustomResolve_Wide.ush"
# elif WIDE_RESOLVE_WIDTH == 2
#  include "WideCustomResolve_Wider.ush"
# elif WIDE_RESOLVE_WIDTH == 3
#  include "WideCustomResolve_Widest.ush"
# else
#  error "Unknown filter width"
# endif

float4 WideCustomResolvePS(float4 Pos : SV_POSITION) : SV_Target0
{
	uint2 P = uint2(ResolveOrigin + Pos.xy);
	return float4(resolve_bspline(P), 0);
}

# endif
#endif
#endif