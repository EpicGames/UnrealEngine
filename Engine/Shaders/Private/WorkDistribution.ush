// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "/Engine/Public/Platform.ush"
#include "WaveOpUtil.ush"

/*
Provides functions to distribute uneven amounts of work uniformly across a wave.
Work won't be distributed wider than the same wave.

The following must be defined:
void DoWork( FWorkContext Context, FWorkSourceType WorkSource, uint LocalItemIndex );
*/

#ifdef GENERATE_WORK

/*
This version can continuously generate work using the function:
uint GenerateWork( FWorkContext Context, uint GroupIndex, inout FWorkSourceType WorkSource, inout bool bDone )
{
	Set WorkSource if there is a valid source of work.
	if( No more work left from this thread )
	{
		bDone = true;
	}
	return NumWorkItems;
}

Once it has a full wave worth of work it consumes it.
*/

groupshared FWorkSourceType	WorkQueueSource[ THREADGROUP_SIZE * 2 ];
groupshared uint			WorkQueueAccum[ THREADGROUP_SIZE * 2 ];
groupshared uint			WorkBoundary[ THREADGROUP_SIZE ];

void DistributeWork( FWorkContext Context, uint GroupIndex )
{
	const uint LaneCount	= WaveGetLaneCount();
	const uint LaneIndex	= GroupIndex &  ( LaneCount - 1 );
	const uint QueueOffset	= GroupIndex & ~( LaneCount - 1 );
	const uint QueueSize	= LaneCount * 2;
	const uint QueueMask	= QueueSize - 1;

#define QUEUE_INDEX(i) ( QueueOffset*2 + ( (i) & QueueMask ) )

	bool bDone = false;

	int WorkRead = 0;
	int WorkWrite = 0;
	int SourceRead = 0;
	int SourceWrite = 0;
	WorkQueueAccum[ QueueOffset*2 + QueueMask ] = 0;

	while( true )
	{
		// Need to queue more work to fill wave?
		while( WorkWrite - WorkRead < LaneCount && WaveActiveAnyTrue( !bDone ) )
		{
			FWorkSourceType WorkSource;
			
			// Generate work and record the source.
			// When sources run out set bDone = true.
			uint NumWorkItems = GenerateWork( Context, GroupIndex, WorkSource, bDone );

			// Queue work
			uint FirstWorkItem = WorkWrite + WavePrefixSum( NumWorkItems );
			uint WorkAccum = FirstWorkItem + NumWorkItems;				// Could use Inclusive sum instead.
			WorkWrite = WaveReadLaneAt( WorkAccum, LaneCount - 1 );

			bool bHasWork = NumWorkItems != 0;
			uint QueueIndex = SourceWrite + WavePrefixCountBits( bHasWork );
			if( bHasWork )
			{
				WorkQueueSource[ QUEUE_INDEX( QueueIndex ) ] = WorkSource;
				WorkQueueAccum[  QUEUE_INDEX( QueueIndex ) ] = WorkAccum;
			}
			SourceWrite += WaveActiveCountBits( bHasWork );
		}

		// Any work left?
		if( WorkWrite == WorkRead )
			break;

		// TODO read and write bytes instead (ds_write_b8, ds_read_u8_d16)
		WorkBoundary[ GroupIndex ] = 0;
		GroupMemoryBarrier();
		
		if( SourceRead + LaneIndex < SourceWrite )
		{
			// Mark the last work item of each source
			uint LastItemIndex = WorkQueueAccum[ QUEUE_INDEX( SourceRead + LaneIndex ) ] - WorkRead - 1;
			if( LastItemIndex < LaneCount )
				WorkBoundary[ QueueOffset + LastItemIndex ] = 1;
		}

		GroupMemoryBarrier();

		bool bIsBoundary = WorkBoundary[ GroupIndex ];

		uint QueueIndex = SourceRead + WavePrefixCountBits( bIsBoundary );

		// Distribute work
		if( WorkRead + LaneIndex < WorkWrite )
		{
			uint FirstWorkItem = WorkQueueAccum[ QUEUE_INDEX( QueueIndex - 1 ) ];
			uint LocalItemIndex = WorkRead + LaneIndex - FirstWorkItem;

			FWorkSourceType WorkSource = WorkQueueSource[ QUEUE_INDEX( QueueIndex ) ];

			DoWork( Context, WorkSource, LocalItemIndex );
		}

		// Did 1 wave of work
		WorkRead = min( WorkRead + LaneCount, WorkWrite );
		SourceRead += WaveActiveCountBits( bIsBoundary );
	}

#undef QUEUE_INDEX
}

#elif 0

bool WaveFlagLaneAt( uint DstIndex, uint SrcIndex )
{
	const uint LaneCount = WaveGetLaneCount();
	
	uint DstMask = 1 << ( DstIndex & 31 );
	uint SrcMask = 1 << ( SrcIndex & 31 );

	DstMask = DstIndex < LaneCount ? DstMask : 0;

	if( LaneCount > 32 )
	{
		bool bDstLow = DstIndex < 32;
		bool bSrcLow = SrcIndex < 32;

		uint2 WaveBits = 0;
		WaveBits.x = WaveActiveBitOr( bDstLow ? DstMask : 0 );
		WaveBits.y = WaveActiveBitOr( bDstLow ? 0 : DstMask );

		return SrcMask & ( bSrcLow ? WaveBits.x : WaveBits.y );
	}
	else
	{
		return WaveActiveBitOr( DstMask ) & SrcMask;
	}
}

// Simpler version where threads can only generate work once.
// This is done before calling DistributeWork so a GenerateWork function doesn't need to be defined.

groupshared FWorkSourceType	WorkQueueSource[ THREADGROUP_SIZE ];
groupshared uint			WorkQueueAccum[ THREADGROUP_SIZE ];
groupshared uint			WorkBoundary[ THREADGROUP_SIZE ];

void DistributeWork( FWorkContext Context, uint GroupIndex, FWorkSourceType WorkSource, uint NumWorkItems )
{
	const uint LaneCount	= WaveGetLaneCount();
	const uint LaneIndex	= GroupIndex &  ( LaneCount - 1 );
	const uint QueueOffset	= GroupIndex & ~( LaneCount - 1 );

	int WorkRead = 0;
	int WorkWrite = 0;
	int SourceRead = 0;

	uint WorkAccum = 0;
	if( WaveActiveAnyTrue( NumWorkItems != 0 ) )
	{
		// Queue work
		uint FirstWorkItem = WavePrefixSum( NumWorkItems );
		WorkAccum = FirstWorkItem + NumWorkItems;				// Could use Inclusive sum instead.
		WorkWrite = WaveReadLaneAt( WorkAccum, LaneCount - 1 );

		bool bHasWork = NumWorkItems != 0;
		uint QueueIndex = WavePrefixCountBits( bHasWork );
		if( bHasWork )
		{
			WorkQueueSource[ QueueOffset + QueueIndex ] = WorkSource;
			WorkQueueAccum[  QueueOffset + QueueIndex ] = WorkAccum;
		}
	}
	
	// Pull work from queue
	while( WorkRead < WorkWrite )
	{
		// TODO read and write bytes instead (ds_write_b8, ds_read_u8_d16)
		WorkBoundary[ GroupIndex ] = 0;
		GroupMemoryBarrier();
		
		// Mark the last work item of each source
		uint LastItemIndex = WorkAccum - WorkRead - 1;
		if( LastItemIndex < LaneCount )
			WorkBoundary[ QueueOffset + LastItemIndex ] = 1;

		GroupMemoryBarrier();

		bool bIsBoundary = WorkBoundary[ GroupIndex ];

		uint QueueIndex = SourceRead + WavePrefixCountBits( bIsBoundary );

		if( WorkRead + LaneIndex < WorkWrite )
		{
			uint FirstWorkItem = QueueIndex > 0 ? WorkQueueAccum[ QueueOffset + QueueIndex - 1 ] : 0;
			uint LocalItemIndex = WorkRead + LaneIndex - FirstWorkItem;

			FWorkSourceType WorkSource = WorkQueueSource[ QueueOffset + QueueIndex ];

			DoWork( Context, WorkSource, LocalItemIndex );
		}

		// Did 1 wave of work
		WorkRead += LaneCount;
		SourceRead += WaveActiveCountBits( bIsBoundary );
	}
}

#else

groupshared uint WorkBoundary[ THREADGROUP_SIZE ];

template< typename FTask >
void DistributeWork( FTask Task, uint GroupIndex, uint NumWorkItems )
{
	const uint LaneCount	= min(WaveGetLaneCount(), THREADGROUP_SIZE);
	const uint LaneIndex	= GroupIndex &  ( LaneCount - 1 );
	const uint QueueOffset	= GroupIndex & ~( LaneCount - 1 );

	int WorkHead = 0;
	int WorkTail = 0;
	int SourceHead = 0;

	uint WorkSource = LaneIndex;
	uint WorkAccum = 0;
	if( WaveActiveAnyTrue( NumWorkItems != 0 ) )
	{
		// Queue work
		uint FirstWorkItem = WavePrefixSum( NumWorkItems );
		WorkAccum = FirstWorkItem + NumWorkItems;				// Could use Inclusive sum instead.
		WorkTail = WaveReadLaneAt( WorkAccum, LaneCount - 1 );

		bool bHasWork = NumWorkItems != 0;
		uint QueueIndex = WavePrefixCountBits( bHasWork );

		// Compact
		if( WaveActiveAnyTrue( NumWorkItems == 0 ) )	// Might know this is impossible
		{
			// Compact LaneIndex
		#if 0//COMPILER_SUPPORTS_WAVE_PERMUTE
			QueueIndex = bHasWork ? QueueIndex : LaneCount - 1;
			WorkSource = WaveWriteLaneAt( QueueIndex, LaneIndex );
		#else
			if( bHasWork )
				WorkBoundary[ QueueOffset + QueueIndex ] = LaneIndex;

			GroupMemoryBarrier();

			WorkSource = WorkBoundary[ GroupIndex ];
		#endif

			WorkAccum  = WaveReadLaneAt( WorkAccum, WorkSource );

			// Push invalid lanes off the end to prevent writes to WorkBoundary and bank conflicts.
			if( LaneIndex >= WaveActiveCountBits( bHasWork ) )
				WorkAccum = WorkTail + LaneCount;
		}
	}
	
	// Pull work from queue
	while( WorkHead < WorkTail )
	{
		// TODO read and write bytes instead (ds_write_b8, ds_read_u8_d16)
		WorkBoundary[ GroupIndex ] = 0;
		GroupMemoryBarrier();
		
		// Mark the last work item of each source
		uint LastItemIndex = WorkAccum - WorkHead - 1;
		if( LastItemIndex < LaneCount )
			WorkBoundary[ QueueOffset + LastItemIndex ] = 1;

		GroupMemoryBarrier();

		bool bIsBoundary = WorkBoundary[ GroupIndex ];

		bool bActive = (WorkHead + LaneIndex < WorkTail);
		uint QueueIndex = select( bActive, SourceHead + WavePrefixCountBits( bIsBoundary ), 0 );	// Make sure inactive threads still point to a valid parent tasks
		uint SourceIndex = WaveReadLaneAt( WorkSource, QueueIndex );
		
		uint FirstWorkItem = select( QueueIndex > 0, WaveReadLaneAt( WorkAccum, QueueIndex - 1 ), 0 );
		uint LocalItemIndex = WorkHead + LaneIndex - FirstWorkItem;

		FTask ChildTask = Task.CreateChild( SourceIndex );
		ChildTask.RunChild( Task, bActive, LocalItemIndex );

		// Did 1 wave of work
		WorkHead += LaneCount;
		SourceHead += WaveActiveCountBits( bIsBoundary );
	}
}

#endif

#if 1
struct FWorkQueueState
{
	uint	ReadOffset;
	uint	WriteOffset;
	int		TaskCount;		// Can temporarily be conservatively higher
};

struct FOutputQueue
{
	RWByteAddressBuffer						DataBuffer;
	RWStructuredBuffer< FWorkQueueState >	StateBuffer;	// Ideally this was GDS but we don't have that API control.

	uint	StateIndex;
	uint	Size;

	uint Add()
	{
		uint WriteOffset;
		WaveInterlockedAddScalar_( StateBuffer[ StateIndex ].WriteOffset, 1, WriteOffset );
		// TODO Copy WriteOffset to TaskCount
		WaveInterlockedAddScalar( StateBuffer[ StateIndex ].TaskCount, 1 );
		return WriteOffset;
	}

	uint DataBuffer_Load(uint Address)
	{
		return DataBuffer.Load(Address);
	}

	uint4 DataBuffer_Load4(uint Address)
	{
		return DataBuffer.Load4(Address);
	}

	void DataBuffer_Store4(uint Address, uint4 Values)
	{
		DataBuffer.Store4(Address, Values);
	}

	FWorkQueueState GetState(uint Index)
	{
		return StateBuffer[Index];
	}
};

struct FInputQueue
{
	ByteAddressBuffer						DataBuffer;
	RWStructuredBuffer< FWorkQueueState >	StateBuffer;	// Ideally this was GDS but we don't have that API control.

	uint	StateIndex;
	uint	Size;

	uint Remove()
	{
		uint ReadOffset;
		WaveInterlockedAddScalar_( StateBuffer[ StateIndex ].ReadOffset, 1, ReadOffset );
		return ReadOffset;
	}

	uint Num()
	{
		return StateBuffer[ StateIndex ].WriteOffset;
	}
};

struct FGlobalWorkQueue
{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
	RWCoherentByteAddressBufferRef                   DataBuffer_Private;
	RWCoherentStructuredBufferRef( FWorkQueueState ) StateBuffer_Private;	// Ideally this was GDS but we don't have that API control.

	RWCoherentByteAddressBuffer                   GetDataBuffer()  { return (RWCoherentByteAddressBuffer)DataBuffer_Private; }
	RWCoherentStructuredBuffer( FWorkQueueState ) GetStateBuffer() { return (RWCoherentStructuredBuffer( FWorkQueueState ))StateBuffer_Private; }
#else
	RWCoherentByteAddressBufferRef                   DataBuffer;
	RWCoherentStructuredBufferRef( FWorkQueueState ) StateBuffer;	// Ideally this was GDS but we don't have that API control.
#endif

	uint	StateIndex;
	uint	Size;

	uint Add()
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentStructuredBuffer(FWorkQueueState) StateBuffer = GetStateBuffer();
#endif

		uint WriteCount = WaveActiveCountBits( true );
		uint WriteOffset = 0;
		if( WaveIsFirstLane() )
		{
			InterlockedAdd( StateBuffer[ StateIndex ].WriteOffset, WriteCount, WriteOffset );
			InterlockedAdd( StateBuffer[ StateIndex ].TaskCount, (int)WriteCount );
		}

		return WaveReadLaneFirst( WriteOffset ) + WavePrefixCountBits( true );
	}

	uint Remove()
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentStructuredBuffer(FWorkQueueState) StateBuffer = GetStateBuffer();
#endif

		uint ReadOffset;
		WaveInterlockedAddScalar_( StateBuffer[ StateIndex ].ReadOffset, 1, ReadOffset );
		return ReadOffset;
	}

	// Only call after current task has completely finished adding work!
	void ReleaseTask()
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentStructuredBuffer(FWorkQueueState) StateBuffer = GetStateBuffer();
#endif

		WaveInterlockedAddScalar( StateBuffer[ StateIndex ].TaskCount, -1 );
	}

	bool IsEmpty()
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentStructuredBuffer(FWorkQueueState) StateBuffer = GetStateBuffer();
#endif

	#if 1
		return StateBuffer[ StateIndex ].TaskCount == 0;
	#else
		uint Count = 0;
		if( WaveIsFirstLane() )
		{
			InterlockedAdd( StateBuffer[ StateIndex ].TaskCount, 0, Count );
		}
		return WaveReadLaneFirst( Count ) == 0;
	#endif
	}

	uint DataBuffer_Load(uint Address)
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentByteAddressBuffer DataBuffer = GetDataBuffer();
#endif
		return DataBuffer.Load(Address);
	}

	uint4 DataBuffer_Load4(uint Address)
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentByteAddressBuffer DataBuffer = GetDataBuffer();
#endif
		return DataBuffer.Load4(Address);
	}

	void DataBuffer_Store4(uint Address, uint4 Values)
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentByteAddressBuffer DataBuffer = GetDataBuffer();
#endif
		DataBuffer.Store4(Address, Values);
	}

	FWorkQueueState GetState(uint Index)
	{
#if COMPILER_NEEDS_GLOBALLYCOHERENT_LOCALS
		RWCoherentStructuredBuffer(FWorkQueueState) StateBuffer = GetStateBuffer();
#endif
		return StateBuffer[Index];
	}
};


template< typename FTask >
void GlobalTaskLoop( FGlobalWorkQueue GlobalWorkQueue )
{
	bool bTaskComplete = true;

	uint TaskReadOffset = 0;
	
	while( true )
	{
		if( WaveActiveAllTrue( bTaskComplete ) )
		{
			TaskReadOffset = GlobalWorkQueue.Remove();
			bTaskComplete = TaskReadOffset >= GlobalWorkQueue.Size;
			if( WaveActiveAllTrue( bTaskComplete ) )
				break;
		}

		FTask Task;
		bool bTaskReady = false;
		if( !bTaskComplete )
		{
			bTaskReady = Task.Load( GlobalWorkQueue, TaskReadOffset );
		}
		
		if( WaveActiveAnyTrue( bTaskReady ) )
		{
			if( bTaskReady )
			{
				Task.Run( GlobalWorkQueue );

				// Clear processed element so we leave the buffer cleared for next pass.
				Task.Clear( GlobalWorkQueue, TaskReadOffset );
				bTaskComplete = true;
			}
		}
		else
		{
			if( GlobalWorkQueue.IsEmpty() )
				break;
			else
			{
				DeviceMemoryBarrier();
				ShaderYield();
			}
		}
	}
}

template< typename FTask >
void GlobalTaskLoopVariable( FGlobalWorkQueue GlobalWorkQueue, uint GroupIndex )
{
	bool bTaskComplete = true;

	uint TaskReadOffset = 0;
	
	while( true )
	{
		if( WaveActiveAllTrue( bTaskComplete ) )
		{
			TaskReadOffset = GlobalWorkQueue.Remove();
			bTaskComplete = TaskReadOffset >= GlobalWorkQueue.Size;
			if( WaveActiveAllTrue( bTaskComplete ) )
				break;
		}

		FTask Task = (FTask)0;
		bool bTaskReady = false;
		if( !bTaskComplete )
		{
			bTaskReady = Task.Load( GlobalWorkQueue, TaskReadOffset );
		}
		
		if( WaveActiveAnyTrue( bTaskReady ) )
		{
			uint NumChildren = 0;
			if( bTaskReady )
			{
				NumChildren = Task.Run();
			}

			DistributeWork( Task, GroupIndex, NumChildren );

			if( bTaskReady )
			{
				// Clear processed element so we leave the buffer cleared for next pass.
				Task.Clear( GlobalWorkQueue, TaskReadOffset );
				bTaskComplete = true;
			}
		}
		else
		{
			if( GlobalWorkQueue.IsEmpty() )
				break;
			else
			{
				DeviceMemoryBarrier();
				ShaderYield();
			}
		}
	}
}

#endif