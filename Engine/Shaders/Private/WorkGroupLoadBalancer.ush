// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

// the define is set by FGPUWorkGroupLoadBalancer::SetShaderDefines which should be used by any CS that uses this.
#ifdef WGLB_ENABLE

#include "Common.ush"
#include "WaveOpUtil.ush"
#include "ComputeShaderUtils.ush"

struct FWorkGroupInfo
{
	uint2 WorkGroupWorkBoundary;
	uint FirstItem;
	uint NumItems;
	uint CarryOverStartOffset;
	uint Payload; // aribitrary 32-bit payload for each workgroup
};

struct FPackedItem
{
	uint Packed;
};

StructuredBuffer<FWorkGroupInfo> WorkGroupInfoBuffer;
StructuredBuffer<FPackedItem> ItemBuffer;
uint NumWorkGroupInfos;
uint NumItems;

struct FWorkGroupItem
{
	bool bHasCarryOver;
	uint NumChildren;
	uint Payload;
	uint BatchPrefixSum;
};

FWorkGroupItem UnpackItem(FPackedItem PackedItem)
{
	FWorkGroupItem Result;

	Result.bHasCarryOver	= (PackedItem.Packed & 1u) != 0u;
	Result.NumChildren		= 1u + ((PackedItem.Packed >> 1u) & WGLB_NUM_ITEM_MASK);
	Result.BatchPrefixSum	= (PackedItem.Packed >> (1u + WGLB_NUM_ITEM_BITS)) & WGLB_PREFIX_BIT_MASK;
	Result.Payload			= (PackedItem.Packed >> (1u + WGLB_NUM_ITEM_BITS + WGLB_PREFIX_BITS));

	return Result;
}

groupshared FPackedItem Items[WGLB_NUM_THREADS_PER_GROUP];

struct FWorkGroupSetup
{
	bool bValid;
	FWorkGroupItem Item;
	FWorkGroupInfo WorkGroupInfo;
	// Work out who I am in expansion of  the item in range [0,WorkItem.Count], may be negative when no work at end of batch
	int LocalItemIndex;
	uint WorkItemIndex;
	uint ChildIndex;
	uint DispatchThreadId;
};

FWorkGroupSetup WorkGroupLoadBalancer_Setup(uint3 GroupId, uint GroupThreadIndex)
{
	FWorkGroupSetup  Setup = (FWorkGroupSetup)0;
	uint WorkGroupIndex = GetUnWrappedDispatchGroupId(GroupId);
	Setup.DispatchThreadId = WorkGroupIndex * WGLB_NUM_THREADS_PER_GROUP + GroupThreadIndex;

	if (WorkGroupIndex >= NumWorkGroupInfos)
	{
		return Setup;
	}

	// Load work description for whole work group
	Setup.WorkGroupInfo = WorkGroupInfoBuffer[WorkGroupIndex];

	FPackedItem PackedItem = (FPackedItem)0u;
	if (GroupThreadIndex < Setup.WorkGroupInfo.NumItems)
	{
		PackedItem = ItemBuffer[Setup.WorkGroupInfo.FirstItem + GroupThreadIndex];
	}
	Items[GroupThreadIndex] = PackedItem;

	GroupMemoryBarrierWithGroupSync();
	Setup.WorkItemIndex = MaskedBitCount(Setup.WorkGroupInfo.WorkGroupWorkBoundary, GroupThreadIndex);
	Setup.Item = UnpackItem(Items[Setup.WorkItemIndex]);
	// Work out who I am in expansion of  the item in range [0,WorkItem.Count], may be negative when no work at end of batch
	Setup.LocalItemIndex = int(GroupThreadIndex - Setup.Item.BatchPrefixSum);
	Setup.ChildIndex = uint(Setup.LocalItemIndex);

	if (Setup.Item.bHasCarryOver)
	{
		Setup.ChildIndex += Setup.WorkGroupInfo.CarryOverStartOffset;
	}
	Setup.bValid = Setup.LocalItemIndex >= 0 && Setup.LocalItemIndex < int(Setup.Item.NumChildren);

	return Setup;
}


uint WorkGroupLoadBalancer_GetNumWorkGroups()
{
	return NumWorkGroupInfos;
}


uint WorkGroupLoadBalancer_GetNumItems()
{
	return NumItems;
}

#endif // WGLB_ENABLE
