// Copyright Epic Games, Inc. All Rights Reserved.

// This header defines the "worldspace vector" FWSVector alias type and its related types
// The underlying implementation can be either tile-offset (LWC*) or double float (DF*)

// Changes made to either branch must be applied to the other as well

#include "/Engine/Private/DoubleFloat.ush"
#include "/Engine/Private/LargeWorldCoordinates.ush"

#if WSVECTOR_IS_TILEOFFSET


#define FWSScalar FLWCScalar
#define FWSVector2 FLWCVector2
#define FWSVector3 FLWCVector3
#define FWSVector4 FLWCVector4

#define FWSScalarDeriv FLWCScalarDeriv
#define FWSVector2Deriv FLWCVector2Deriv
#define FWSVector3Deriv FLWCVector3Deriv
#define FWSVector4Deriv FLWCVector4Deriv
#define FWSMatrix FLWCMatrix
#define FWSInverseMatrix FLWCInverseMatrix

#define MakeWSScalar MakeLWCScalar
#define MakeWSVector2 MakeLWCVector2
#define MakeWSVector3 MakeLWCVector3
#define MakeWSVector4 MakeLWCVector4
#define MakeWSVector MakeLWCVector
#define MakeWSMatrix MakeLWCMatrix
#define MakeWSMatrix4x3 MakeLWCMatrix4x3
#define MakeWSInverseMatrix MakeLWCInverseMatrix
#define MakeWSInverseMatrix4x3 MakeLWCInverseMatrix4x3

FWSScalar LWCToWS(FLWCScalar In) { return In; }
FWSScalar DFToWS(FDFScalar In) { return DFToTileOffset(In); }
FWSScalar DFFastToWS(FDFScalar In) { return DFFastToTileOffset(In); }
FLWCScalar WSToLWC(FWSScalar In) { return In; }
FDFScalar WSToDF(FWSScalar In) { return DFFromTileOffset(In); }

FWSVector2 LWCToWS(FLWCVector2 In) { return In; }
FWSVector2 DFToWS(FDFVector2 In) { return DFToTileOffset(In); }
FWSVector2 DFFastToWS(FDFVector2 In) { return DFFastToTileOffset(In); }
FLWCVector2 WSToLWC(FWSVector2 In) { return In; }
FDFVector2 WSToDF(FWSVector2 In) { return DFFromTileOffset(In); }

FWSVector3 LWCToWS(FLWCVector3 In) { return In; }
FWSVector3 DFToWS(FDFVector3 In) { return DFToTileOffset(In); }
FWSVector3 DFFastToWS(FDFVector3 In) { return DFFastToTileOffset(In); }
FLWCVector3 WSToLWC(FWSVector3 In) { return In; }
FDFVector3 WSToDF(FWSVector3 In) { return DFFromTileOffset(In); }

FWSVector4 LWCToWS(FLWCVector4 In) { return In; }
FWSVector4 DFToWS(FDFVector4 In) { return DFToTileOffset(In); }
FWSVector4 DFFastToWS(FDFVector4 In) { return DFFastToTileOffset(In); }
FLWCVector4 WSToLWC(FWSVector4 In) { return In; }
FDFVector4 WSToDF(FWSVector4 In) { return DFFromTileOffset(In); }

FWSMatrix LWCToWS(FLWCMatrix In) { return In; }
FWSMatrix DFToWS(FDFMatrix In) { return DFToTileOffset(In); }
FWSMatrix DFFastToWS(FDFMatrix In) { return DFFastToTileOffset(In); }
FLWCMatrix WSToLWC(FWSMatrix In) { return In; }
FDFMatrix WSToDF(FWSMatrix In) { return DFFromTileOffset(In); }

FWSInverseMatrix LWCToWS(FLWCInverseMatrix In) { return In; }
FWSInverseMatrix DFToWS(FDFInverseMatrix In) { return DFToTileOffset(In); }
FWSInverseMatrix DFFastToWS(FDFInverseMatrix In) { return DFFastToTileOffset(In); }
FLWCInverseMatrix WSToLWC(FWSInverseMatrix In) { return In; }
FDFInverseMatrix WSToDF(FWSInverseMatrix In) { return DFFromTileOffset(In); }

#define WSGetComponent LWCGetComponent
#define WSGetX LWCGetX
#define WSGetY LWCGetY
#define WSGetZ LWCGetZ
#define WSGetW LWCGetW
#define WSSwizzle LWCSwizzle
#define WSDemote LWCToFloat
#define WSToFloat3x3 LWCToFloat3x3
#define WSPromote LWCPromote
#define WSPromoteInverse LWCPromoteInverse
#define WSMultiply LWCMultiply
#define WSMultiplyByPow2 LWCMultiply
#define WSMultiplyDemote LWCMultiply
#define WSMultiplyVector LWCMultiplyVector
#define WSMultiplyTranslation LWCMultiplyTranslation
#define WSGetOrigin LWCGetOrigin
#define WSSetOrigin LWCSetOrigin

#define WSSqrtDemote LWCSqrt
#define WSRsqrtDemote LWCRsqrt
#define WSRcpDemote LWCRcp

#define WSGreater LWCGreater
#define WSGreaterEqual LWCGreaterEqual
#define WSLess LWCLess
#define WSLessEqual LWCLessEqual
#define WSEquals LWCEquals
#define WSEqualsApprox LWCEqualsApprox
#define WSNotEquals LWCNotEquals

#define WSDdxDemote LWCDdx
#define WSDdyDemote LWCDdy

#define WSAdd LWCAdd
#define WSAddDemote(A, B) LWCToFloat(LWCAdd(A, B))
#define WSSubtract LWCSubtract
#define WSSubtractDemote(A, B) LWCToFloat(LWCSubtract(A, B))

#define WSSelect LWCSelect
#define WSNegate LWCNegate
#define WSFracDemote LWCFrac
#define WSFloor LWCFloor
#define WSCeil LWCCeil
#define WSRound LWCRound
#define WSTrunc LWCTrunc
#define WSSign LWCSign
#define WSSaturateDemote LWCSaturate
#define WSDivide LWCDivide
#define WSDivideByPow2 LWCDivide
#define WSLerp LWCLerp
#define WSFmodDemote LWCFmod
#define WSModuloDemote LWCModulo

#define WSSin LWCSin
#define WSCos LWCCos
#define WSTan LWCTan
#define WSASin LWCASin
#define WSACos LWCACos
#define WSATan LWCATan

#define WSSmoothStepDemote LWCSmoothStep

#define WSMin LWCMin
#define WSMax LWCMax
#define WSAbs LWCAbs
#define WSStep LWCStep
#define WSClamp LWCClamp

#define WSVectorSum LWCVectorSum
#define WSDot LWCDot
#define WSLength LWCLength
#define WSRcpLengthDemote LWCRcpLength
#define WSNormalizeDemote LWCNormalize
#define WSDistance LWCDistance


#elif WSVECTOR_IS_DOUBLEFLOAT


#define FWSScalar FDFScalar
#define FWSVector2 FDFVector2
#define FWSVector3 FDFVector3
#define FWSVector4 FDFVector4

#define FWSScalarDeriv FDFScalarDeriv
#define FWSVector2Deriv FDFVector2Deriv
#define FWSVector3Deriv FDFVector3Deriv
#define FWSVector4Deriv FDFVector4Deriv
#define FWSMatrix FDFMatrix
#define FWSInverseMatrix FDFInverseMatrix

#define MakeWSScalar MakeDFScalar
#define MakeWSVector2 MakeDFVector2
#define MakeWSVector3 MakeDFVector3
#define MakeWSVector4 MakeDFVector4
#define MakeWSVector MakeDFVector
#define MakeWSMatrix MakeDFMatrix
#define MakeWSMatrix4x3 MakeDFMatrix4x3
#define MakeWSInverseMatrix MakeDFInverseMatrix
#define MakeWSInverseMatrix4x3 MakeDFInverseMatrix4x3

FWSScalar LWCToWS(FLWCScalar In) { return DFFromTileOffset(In); }
FWSScalar DFToWS(FDFScalar In) { return In; }
FWSScalar DFFastToWS(FDFScalar In) { return In; }
FLWCScalar WSToLWC(FWSScalar In) { return DFToTileOffset(In); }
FDFScalar WSToDF(FWSScalar In) { return In; }

FWSVector2 LWCToWS(FLWCVector2 In) { return DFFromTileOffset(In); }
FWSVector2 DFToWS(FDFVector2 In) { return In; }
FWSVector2 DFFastToWS(FDFVector2 In) { return In; }
FLWCVector2 WSToLWC(FWSVector2 In) { return DFToTileOffset(In); }
FDFVector2 WSToDF(FWSVector2 In) { return In; }

FWSVector3 LWCToWS(FLWCVector3 In) { return DFFromTileOffset(In); }
FWSVector3 DFToWS(FDFVector3 In) { return In; }
FWSVector3 DFFastToWS(FDFVector3 In) { return In; }
FLWCVector3 WSToLWC(FWSVector3 In) { return DFToTileOffset(In); }
FDFVector3 WSToDF(FWSVector3 In) { return In; }

FWSVector4 LWCToWS(FLWCVector4 In) { return DFFromTileOffset(In); }
FWSVector4 DFToWS(FDFVector4 In) { return In; }
FWSVector4 DFFastToWS(FDFVector4 In) { return In; }
FLWCVector4 WSToLWC(FWSVector4 In) { return DFToTileOffset(In); }
FDFVector4 WSToDF(FWSVector4 In) { return In; }

FWSMatrix LWCToWS(FLWCMatrix In) { return DFFromTileOffset(In); }
FWSMatrix DFToWS(FDFMatrix In) { return In; }
FWSMatrix DFFastToWS(FDFMatrix In) { return In; }
FLWCMatrix WSToLWC(FWSMatrix In) { return DFToTileOffset(In); }
FDFMatrix WSToDF(FWSMatrix In) { return In; }

FWSInverseMatrix LWCToWS(FLWCInverseMatrix In) { return DFFromTileOffset(In); }
FWSInverseMatrix DFToWS(FDFInverseMatrix In) { return In; }
FWSInverseMatrix DFFastToWS(FDFInverseMatrix In) { return In; }
FLWCInverseMatrix WSToLWC(FWSInverseMatrix In) { return DFToTileOffset(In); }
FDFInverseMatrix WSToDF(FWSInverseMatrix In) { return In; }

#define WSGetComponent DFGetComponent
#define WSGetX DFGetX
#define WSGetY DFGetY
#define WSGetZ DFGetZ
#define WSGetW DFGetW
#define WSSwizzle DFSwizzle
#define WSDemote DFDemote
#define WSToFloat3x3 DFToFloat3x3
#define WSPromote DFPromote
#define WSPromoteInverse DFPromoteInverse
#define WSMultiply DFMultiply
#define WSMultiplyByPow2 DFMultiplyByPow2
#define WSMultiplyDemote DFMultiplyDemote
#define WSMultiplyVector DFMultiplyVector
#define WSMultiplyTranslation DFMultiplyTranslation
#define WSGetOrigin DFGetOrigin
#define WSSetOrigin DFSetOrigin

#define WSSqrtDemote DFSqrtDemote
#define WSRsqrtDemote DFRsqrtDemote
#define WSRcpDemote DFRcpDemote

#define WSGreater DFGreater
#define WSGreaterEqual DFGreaterEqual
#define WSLess DFLess
#define WSLessEqual DFLessEqual
#define WSEquals DFEquals
#define WSEqualsApprox DFEqualsApprox
#define WSNotEquals DFNotEquals

#define WSDdxDemote DFDdxDemote
#define WSDdyDemote DFDdyDemote

#define WSAdd DFFastAdd
#define WSAddDemote DFFastAddDemote
#define WSSubtract DFFastSubtract
#define WSSubtractDemote(A, B) DFFastSubtractDemote(A, B)

#define WSSelect DFSelect
#define WSNegate DFNegate
#define WSFracDemote DFFracDemote
#define WSFloor DFFloor
#define WSCeil DFCeil
#define WSRound DFRound
#define WSTrunc DFTrunc
#define WSSign DFSign
#define WSSaturateDemote DFSaturateDemote
#define WSDivide DFFastDivide
#define WSDivideByPow2 DFDivideByPow2
#define WSLerp DFLerp
#define WSFmodDemote DFFmodDemote
#define WSModuloDemote DFModuloDemote

#define WSSin DFSin
#define WSCos DFCos
#define WSTan DFTan
#define WSASin DFASin
#define WSACos DFACos
#define WSATan DFATan

#define WSSmoothStepDemote DFSmoothStepDemote

#define WSMin DFMin
#define WSMax DFMax
#define WSAbs DFAbs
#define WSStep DFStep
#define WSClamp DFClamp

#define WSVectorSum DFVectorSum
#define WSDot DFDot
#define WSLength DFLength
#define WSRcpLengthDemote DFRcpLengthDemote
#define WSNormalizeDemote DFNormalizeDemote
#define WSDistance DFDistance


#endif

#define WSHackToFloat WSDemote

#define DEFINE_FP32_NOPS(FFloatType) \
	FFloatType LWCToWS(FFloatType In) { return In; } \
	FFloatType DFToWS(FFloatType In) { return In; } \
	FFloatType WSToLWC(FFloatType In) { return In; } \
	FFloatType WSToDF(FFloatType In) { return In; }

DEFINE_FP32_NOPS(float)
DEFINE_FP32_NOPS(float2)
DEFINE_FP32_NOPS(float3)
DEFINE_FP32_NOPS(float4)

#undef DEFINE_FP32_NOPS
