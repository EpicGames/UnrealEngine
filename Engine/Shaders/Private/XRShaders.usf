// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common.ush"

Texture2D InTexture;
TextureCube InTextureCube;
SamplerState InTextureSampler;
float4x4 FaceToSample;

void MainForCubemap(
	FScreenVertexOutput Input,
	out float4 OutColor : SV_Target0
	)
{
	float2 uv = Input.UV * 2. - 1.0;
	float3 sampleUVW = mul(float4(uv, 1.0,1.0), FaceToSample).xyz;
#if VULKAN_PROFILE
	sampleUVW.y = -sampleUVW.y;
#endif
	OutColor = TextureCubeSample(InTextureCube, InTextureSampler, sampleUVW);
}
