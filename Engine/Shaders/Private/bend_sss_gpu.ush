#pragma once

// Copyright 2023 Sony Interactive Entertainment.
//
// Licensed under the Apache License, Version 2.0 (the "License");
// you may not use this file except in compliance with the License.
// You may obtain a copy of the License at
//
//     http://www.apache.org/licenses/LICENSE-2.0
//
// Unless required by applicable law or agreed to in writing, software
// distributed under the License is distributed on an "AS IS" BASIS,
// WITHOUT WARRANTIES OR CONDITIONS OF ANY KIND, either express or implied.
// See the License for the specific language governing permissions and
// limitations under the License.

// If you have feedback, or found this code useful, we'd love to hear from you.
// https://www.bendstudio.com
// https://www.twitter.com/bendstudio
// 
// We are *always* looking for talented graphics and technical programmers!
// https://www.bendstudio.com/careers


// Common screen space shadow projection code (GPU):
//--------------------------------------------------------------

// The main shadow generation function is WriteScreenSpaceShadow(), it will read a depth texture, and write to a shadow texture
// This code is setup to target DX12 DXC shader compiler, but has also been tested on PS5 with appropriate API remapping. 
// It can compile to DX11, but requires some modifications (e.g., early-out's use of wave intrinsics is not supported in DX11).
// Note; you can customize the 'EarlyOutPixel' function to perform custom early-out logic to optimize this shader.

// The following Macros must be defined in the compute shader file before including this header:
//		
//		
//		#define WAVE_SIZE 64						// Wavefront size of the compute shader running this code. 
//													// numthreads[WAVE_SIZE, 1, 1]
//													// Only tested with 64.
//		
//		#define SAMPLE_COUNT 60						// Number of shadow samples per-pixel.
//													// Determines overall cost, as this value controls the length of the shadow (in pixels).
//													// The number of texture-reads performed per-thread will be (SAMPLE_COUNT / WAVE_SIZE + 2) * 2.
//													// Recommended starting value is 60 (This would be 4 reads per thread if WAVE_SIZE is 64). A value of 64 would require 6 reads.
//		
//		// Not all shadow samples are treated the same:
//		//	The bulk of samples will average together in to groups of 4, to produce a slightly smoothed result (so one sample cannot fully show the pixel)
//		//	However, the samples very close to the start pixel can optionally be forced to disable this averaging, so a single sample can fully shadow the pixel (HardShadowSamples)
//		//	Plus, a number of the last (most distant) samples can (for a small cost) apply a fade-out effect to soften a hash shadow cutoff (FadeOutSamples)
//		
//		#define HARD_SHADOW_SAMPLES 4				// Number of initial shadow samples that will produce a hard shadow, and not perform sample-averaging.
//													// This trades aliasing for grounding pixels very close to the shadow caster.
//													// Recommended starting value: 4
//		
//		#define FADE_OUT_SAMPLES 8					// Number of samples that will fade out at the end of the shadow (for a minor cost).
//													// Recommended starting value: 8
//		
//		#define TRACK_HIT_UV 1						// Whether to track Hit UV while tracing (causes significant overhead on some platforms).
//													// If disabled outHitUV will be (0,0)


#if defined(__HLSL_VERSION) || defined(__hlsl_dx_compiler) || 1
	
	#define USE_HALF_PIXEL_OFFSET 1		// Apply a 0.5 texel offset when sampling a texture. Toggle this macro if the output shadow has odd, regular grid-like artefacts.

	// HLSL enforces that a pixel offset in a Sample() call must be a compile time constant, which isn't always required - and in some cases can give a small perf boost if used.
	#define USE_UV_PIXEL_BIAS 1			// Use Sample(uv + bias) instead of Sample(uv, bias)

#endif


// Forward declare:
// Generate the shadow
//	Call this function from a compute shader with thread dimensions: numthreads[WAVE_SIZE, 1, 1]
// 
//	(int3)	inGroupID:			Compute shader group id register (SV_GroupID)
//	(int)	inGroupThreadId:	Compute shader group thread id register (SV_GroupThreadID)
void WriteScreenSpaceShadow(struct DispatchParameters inParameters, int3 inGroupID, int inGroupThreadID, out float outResult, out int2 outResultXY, out float2 outHitUV);



// This is the list of runtime properties to pass to the shader
// Wherever possible, it is highly recommended to have these values be compile-time constants
struct DispatchParameters
{
	// Visual configuration:
	// These values will require manual tuning.
	// All shadow computation is performed in non-linear depth space (not in world space), so tuned value choices will depend on scene depth distribution (as determined by the Projection Matrix setup).

	float SurfaceThickness;				// This is the assumed thickness of each pixel for shadow-casting, measured as a percentage of the difference in non-linear depth between the sample and FarDepthValue.
										// Recommended starting value: 0.005 (0.5%)

	float BilinearThreshold;			// Percentage threshold for determining if the difference between two depth values represents an edge, and should not perform interpolation.
										// To tune this value, set 'DebugOutputEdgeMask' to true to visualize where edges are being detected.
										// Recommended starting value: 0.02 (2%)

	float ShadowContrast;				// A contrast boost is applied to the transition in/out of shadow.
										// Recommended starting value: 2 or 4. Values >= 1 are valid.

	bool IgnoreEdgePixels;				// If an edge is detected, the edge pixel will not contribute to the shadow.
										// If a very flat surface is being lit and rendered at an grazing angles, the edge detect may incorrectly detect multiple 'edge' pixels along that flat surface.
										// In these cases, the grazing angle of the light may subsequently produce aliasing artefacts in the shadow where these incorrect edges were detected.
										// Setting this value to true would mean that those pixels would not cast a shadow, however it can also thin out otherwise valid shadows, especially on foliage edges.
										// Recommended starting value: false, unless typical scenes have numerous large flat surfaces, in which case true.

	bool UsePrecisionOffset;			// A small offset is applied to account for an imprecise depth buffer (recommend off)


	bool BilinearSamplingOffsetMode;	// There are two modes to compute bilinear samples for shadow depth:
										// true = sampling points for pixels are offset to the wavefront shared ray, shadow depths and starting depths are the same. Can project more jagged/aliased shadow lines in some cases.
										// false = sampling points for pixels are not offset and start from pixel centers. Shadow depths are biased based on depth gradient across the current pixel bilinear sample. Has more issues in back-face / grazing areas.
										// Both modes have subtle visual differences, which may / may not exaggerate depth buffer aliasing that gets projected in to the shadow.
										// Evaluating the visual difference between each mode is recommended, then hard-coding the mode used to optimize the shader.
										// Recommended starting value: false

	// Debug views
	bool DebugOutputEdgeMask;			// Use this to visualize edges, for tuning the 'BilinearThreshold' value.
	bool DebugOutputThreadIndex;		// Debug output to visualize layout of compute threads
	bool DebugOutputWaveIndex;			// Debug output to visualize layout of compute wavefronts, useful to sanity check the Light Coordinate is being computed correctly.

	// Culling / Early out:
	float2 DepthBounds;					// Depth Bounds (min, max) for the on-screen volume of the light. Typically (0,1) for directional lights. Only used when 'UseEarlyOut' is true.

	bool UseEarlyOut;					// Set to true to early-out when depth values are not within [DepthBounds] - otherwise DepthBounds is unused
										// [Optionally customize the 'EarlyOutPixel()' function to perform your own early-out logic, e.g. skipping pixels that a shadow map indicates are already fully occluded]
										// This can dramatically reduce cost when only a small portion of the pixels need a shadow term (e.g., cull out sky pixels), however it does have some overhead (~15%) in worst-case where nothing early-outs
										// Note; Early-out is most efficient when WAVE_SIZE matches the hardware wavefront size - otherwise cross wave communication is required.


	// Set sensible starting tuning values
	void SetDefaults()
	{
		SurfaceThickness			= 0.005;
		BilinearThreshold			= 0.02;
		ShadowContrast				= 4;
		IgnoreEdgePixels			= false;
		UsePrecisionOffset			= false;
		BilinearSamplingOffsetMode	= false;
		DebugOutputEdgeMask			= false;
		DebugOutputThreadIndex		= false;
		DebugOutputWaveIndex		= false;
		DepthBounds					= float2(0,1);
		UseEarlyOut					= false;
	}

	// Runtime data returned from BuildDispatchList():
	float4 LightCoordinate;				// Values stored in DispatchList::LightCoordinate_Shader by BuildDispatchList()
	int2 WaveOffset;					// Values stored in DispatchData::WaveOffset_Shader by BuildDispatchList()

	// Renderer Specific Values:
	float FarDepthValue;				// Set to the Depth Buffer Value for the far clip plane, as determined by renderer projection matrix setup (typically 0).
	float NearDepthValue;				// Set to the Depth Buffer Value for the near clip plane, as determined by renderer projection matrix setup (typically 1).

	// Sampling data:
	float2 InvDepthTextureSize;			// Inverse of the texture dimensions for 'DepthTexture' (used to convert from pixel coordinates to UVs)
										// If 'PointBorderSampler' is an Unnormalized sampler, then this value can be hard-coded to 1.
										// The 'USE_HALF_PIXEL_OFFSET' macro might need to be defined if sampling at exact pixel coordinates isn't precise (e.g., if odd patterns appear in the shadow).

	Texture2D DepthTexture;				// Depth Buffer Texture (rasterized non-linear depth)

	SamplerState PointBorderSampler;	// A point sampler, with Wrap Mode set to Clamp-To-Border-Color (D3D12_TEXTURE_ADDRESS_MODE_BORDER), and Border Color set to "FarDepthValue" (typically zero), or some other far-depth value out of DepthBounds.
										// If you have issues where invalid shadows are appearing from off-screen, it is likely that this sampler is not correctly setup
};


#if !defined(WAVE_SIZE) || !defined(SAMPLE_COUNT) || !defined(HARD_SHADOW_SAMPLES) || !defined(FADE_OUT_SAMPLES) || !defined(TRACK_HIT_UV)
	#error Before including bend_sss_gpu.h, four macros must be defined to configure the shader compile: WAVE_SIZE, SAMPLE_COUNT, HARD_SHADOW_SAMPLES, FADE_OUT_SAMPLES and TRACK_HIT_UV. See the top of this file for details.
#else

	static bool EarlyOutPixel(struct DispatchParameters inParameters, int2 pixel_xy, float depth)
	{
		//OPTIONAL TODO; customize this function to return true if the pixel should early-out for custom reasons. E.g., A shadow map pass already found the pixel was in shadow / backfaced, etc.
		// Recommended to keep this code very simple!

		// Example:
		// return inParameters.CustomShadowMapTerm[pixel_xy] == 0;

#if 0 // disabled to workaround compilation issues on some platforms
		(void)pixel_xy;	//unused by this implementation, avoid potential compiler warning.
#endif

		// The compiled code will be more optimal if the 'depth' value is not referenced.
		return depth >= inParameters.DepthBounds.y || depth <= inParameters.DepthBounds.x;
	}

	// Gets the start pixel coordinates for the pixels in the wavefront
	// Also returns the delta to get to the next pixel after WAVE_COUNT pixels along the ray
	static void ComputeWavefrontExtents(DispatchParameters inParameters, int3 inGroupID, uint inGroupThreadID, out float2 outDeltaXY, out float2 outPixelXY, out float outPixelDistance, out bool outMajorAxisX)
	{
		int2 xy = inGroupID.yz * WAVE_SIZE + inParameters.WaveOffset.xy;

		//integer light position / fractional component
		float2 light_xy = floor(inParameters.LightCoordinate.xy) + 0.5;
		float2 light_xy_fraction = inParameters.LightCoordinate.xy - light_xy;
		bool reverse_direction = inParameters.LightCoordinate.w > 0.0f;

		int2 sign_xy = sign(xy);
		bool horizontal = abs(xy.x + sign_xy.y) < abs(xy.y - sign_xy.x);

		int2 axis;
		axis.x = horizontal ? (+sign_xy.y) : (0);
		axis.y = horizontal ? (0) : (-sign_xy.x);

		// Apply wave offset
		xy = axis * (int)inGroupID.x + xy;
		float2 xy_f = (float2)xy;

		// For interpolation to the light center, we only really care about the larger of the two axis
		bool x_axis_major = abs(xy_f.x) > abs(xy_f.y);
		float major_axis = x_axis_major ? xy_f.x : xy_f.y;

		float major_axis_start = abs(major_axis);
		float major_axis_end = abs(major_axis) - (float)WAVE_SIZE;

		float ma_light_frac = x_axis_major ? light_xy_fraction.x : light_xy_fraction.y;
		ma_light_frac = major_axis > 0 ? -ma_light_frac : ma_light_frac;

		// back in to screen direction
		float2 start_xy = xy_f + light_xy;

		// For the very inner most ring, we need to interpolate to a pixel centered UV, so the UV->pixel rounding doesn't skip output pixels
		float2 end_xy = lerp(inParameters.LightCoordinate.xy, start_xy, (major_axis_end + ma_light_frac) / (major_axis_start + ma_light_frac));

		// The major axis should be a round number
		float2 xy_delta = (start_xy - end_xy);

		// Inverse the read order when reverse direction is true
		float thread_step = (float)(inGroupThreadID ^ (reverse_direction ? 0 : (WAVE_SIZE - 1)));

		float2 pixel_xy = lerp(start_xy, end_xy, thread_step / (float)WAVE_SIZE);
		float pixel_distance = major_axis_start - thread_step + ma_light_frac;

		outPixelXY = pixel_xy;
		outPixelDistance = pixel_distance;
		outDeltaXY = xy_delta;
		outMajorAxisX = x_axis_major;
	}

	// Number of bilinear sample reads performed per-thread
	#define READ_COUNT (SAMPLE_COUNT / WAVE_SIZE + 2)

	// Common shared data
	groupshared float DepthData[READ_COUNT * WAVE_SIZE];
	groupshared bool LdsEarlyOut;

	#define DEBUG_HIT_UV (0)
	#define PACKED_HIT_INDEX (1 && !DEBUG_HIT_UV)

#if DEBUG_HIT_UV
	groupshared float2 UVData[READ_COUNT * WAVE_SIZE];
#endif

	// Generate the shadow
	//	Call this function from a compute shader with thread dimensions: numthreads[WAVE_SIZE, 1, 1]
	// 
	//	(int3)	inGroupID:			Compute shader group id register (SV_GroupID)
	//	(int)	inGroupThreadId:	Compute shader group thread id register (SV_GroupThreadID)
	void WriteScreenSpaceShadow(DispatchParameters inParameters, int3 inGroupID, int inGroupThreadID, out float outResult, out int2 outResultXY, out float2 outHitUV)
	{
		outResult = 0;
		outResultXY = 0;
		outHitUV = 0;

		float2 xy_delta;
		float2 pixel_xy;
		float pixel_distance;
		bool x_axis_major;	// major axis is x axis? abs(xy_delta.x) > abs(xy_delta.y).

		ComputeWavefrontExtents(inParameters, (int3)inGroupID, inGroupThreadID.x, xy_delta, pixel_xy, pixel_distance, x_axis_major);

		// Read in the depth values
		float sampling_depth[READ_COUNT];
		float shadowing_depth[READ_COUNT];
		float depth_thickness_scale[READ_COUNT];
		float sample_distance[READ_COUNT];
	#if DEBUG_HIT_UV
		float2 sample_uv[READ_COUNT];
	#endif

		const float direction = -inParameters.LightCoordinate.w;
		const float z_sign = inParameters.NearDepthValue > inParameters.FarDepthValue ? -1 : +1;
		
		const float2 start_xy = pixel_xy;

		int i;
		bool is_edge = false;
		bool skip_pixel = false;
		float2 write_xy = floor(pixel_xy);

		[unroll] for (i = 0; i < READ_COUNT; i++)
		{
			// We sample depth twice per pixel per sample, and interpolate with an edge detect filter
			// Interpolation should only occur on the minor axis of the ray - major axis coordinates should be at pixel centers
			float2 read_xy = floor(pixel_xy);
			float minor_axis = x_axis_major ? pixel_xy.y : pixel_xy.x;

			// If a pixel has been detected as an edge, then optionally (inParameters.IgnoreEdgePixels) don't include it in the shadow
			const float edge_skip = 1e20;	// if edge skipping is enabled, apply an extreme value/blend on edge samples to push the value out of range

			float2 depths;
			float bilinear = frac(minor_axis) - 0.5;

		#if USE_HALF_PIXEL_OFFSET
			read_xy += 0.5;
		#endif

		#if USE_UV_PIXEL_BIAS
			float bias = bilinear > 0 ? 1 : -1;
			float2 offset_xy = float2(x_axis_major ? 0 : bias, x_axis_major ? bias : 0);

			// HLSL enforces that a pixel offset is a compile-time constant, which isn't strictly required (and can sometimes be a bit faster)
			// So this fallback will use a manual uv offset instead
			depths.x = inParameters.DepthTexture.SampleLevel(inParameters.PointBorderSampler, read_xy * inParameters.InvDepthTextureSize, 0).r;
			depths.y = inParameters.DepthTexture.SampleLevel(inParameters.PointBorderSampler, (read_xy + offset_xy) * inParameters.InvDepthTextureSize, 0).r;
		#else
			int bias = bilinear > 0 ? 1 : -1;
			int2 offset_xy = int2(x_axis_major ? 0 : bias, x_axis_major ? bias : 0);

			depths.x = inParameters.DepthTexture.SampleLevel(inParameters.PointBorderSampler, read_xy * inParameters.InvDepthTextureSize, 0).r;
			depths.y = inParameters.DepthTexture.SampleLevel(inParameters.PointBorderSampler, read_xy * inParameters.InvDepthTextureSize, 0, offset_xy).r;
		#endif

			// Depth thresholds (bilinear/shadow thickness) are based on a fractional ratio of the difference between sampled depth and the far clip depth
			depth_thickness_scale[i] = abs(inParameters.FarDepthValue - depths.x);

			// If depth variance is more than a specific threshold, then just use point filtering
			bool use_point_filter = abs(depths.x - depths.y) > depth_thickness_scale[i] * inParameters.BilinearThreshold;

			// Store for debug output when inParameters.DebugOutputEdgeMask is true
			if (i == 0) is_edge = use_point_filter;

			if (inParameters.BilinearSamplingOffsetMode)
			{
				bilinear = use_point_filter ? 0 : bilinear;
				//both shadow depth and starting depth are the same in this mode, unless shadow skipping edges
				sampling_depth[i] = lerp(depths.x, depths.y, abs(bilinear));
				shadowing_depth[i] = (inParameters.IgnoreEdgePixels && use_point_filter) ? edge_skip : sampling_depth[i];
			}
			else
			{
				// The pixel starts sampling at this depth
				sampling_depth[i] = depths.x;

				float edge_depth = inParameters.IgnoreEdgePixels ? edge_skip : depths.x;
				// Any sample in this wavefront is possibly interpolated towards the bilinear sample
				// So use should use a shadowing depth that is further away, based on the difference between the two samples
				float shadow_depth = depths.x + abs(depths.x - depths.y) * z_sign;

				// Shadows cast from this depth
				shadowing_depth[i] = use_point_filter ? edge_depth : shadow_depth;
			}

			// Store for later
			sample_distance[i] = pixel_distance + (WAVE_SIZE * i) * direction;
		#if DEBUG_HIT_UV
			sample_uv[i] = read_xy * inParameters.InvDepthTextureSize;
		#endif

			// Iterate to the next pixel along the ray. This will be WAVE_SIZE pixels along the ray...
			pixel_xy += xy_delta * direction;
		}

		// Using early out, and no debug mode is enabled?
		if (inParameters.UseEarlyOut && (inParameters.DebugOutputWaveIndex == false && inParameters.DebugOutputThreadIndex == false && inParameters.DebugOutputEdgeMask == false))
		{
			// read the depth of the pixel we are shadowing, and early-out
			// The compiler will typically rearrange this code to put it directly after the first depth read
			skip_pixel = EarlyOutPixel(inParameters, (int2)write_xy, sampling_depth[0]);

			// are all threads in this wave out of bounds?
			bool early_out = WaveActiveAnyTrue(!skip_pixel) == false;

			// WaveGetLaneCount returns the hardware wave size
			if (WaveGetLaneCount() == WAVE_SIZE)
			{
				// Optimal case:
				// If each wavefront is just a single wave, then we can trivially early-out.
				if (early_out == true)
					return;
			}
			else
			{
				// This wavefront is made up of multiple small waves, so we need to coordinate them for all to early-out together.
				// Doing this can make the worst case (all pixels drawn) a bit more expensive (~15%), but the best-case (all early-out) is typically 2-3x better.
				LdsEarlyOut = true;

				GroupMemoryBarrierWithGroupSync();

				[branch] if (early_out == false)
					LdsEarlyOut = false;

				GroupMemoryBarrierWithGroupSync();

				[branch] if (LdsEarlyOut)
					return;
			}
		}

		// Write the shadow depths to LDS
		[unroll] for (i = 0; i < READ_COUNT; i++)
		{
			// Perspective correct the shadowing depth, in this space, all light rays are parallel
			float stored_depth = (shadowing_depth[i] - inParameters.LightCoordinate.z) / sample_distance[i];

			if (i != 0)
			{
				// For pixels within sampling distance of the light, it is possible that sampling will
				// overshoot the light coordinate for extended reads. We want to ignore these samples
				stored_depth = sample_distance[i] > 0 ? stored_depth : 1e10;
			}

			// Store the depth values in groupshared
			int idx = (i * WAVE_SIZE) + inGroupThreadID.x;
			DepthData[idx] = stored_depth;
		#if DEBUG_HIT_UV
			UVData[idx] = sample_uv[i];
		#endif
		}

		// Sync wavefronts now groupshared DepthData is written
		GroupMemoryBarrierWithGroupSync();

		// If the starting depth isn't in depth bounds, then we don't need a shadow
		if (skip_pixel)
			return;

		float start_depth = sampling_depth[0];

		// lerp away from far depth by a tiny fraction?
		if (inParameters.UsePrecisionOffset)
			start_depth = lerp(start_depth, inParameters.FarDepthValue, -1.0 / 0xFFFF);

		// perspective correct the depth
		start_depth = (start_depth - inParameters.LightCoordinate.z) / sample_distance[0];

		// Start by reading the next value
		int sample_index = inGroupThreadID.x + 1;

#if TRACK_HIT_UV && PACKED_HIT_INDEX
		uint4 shadow_value_packed = asuint(1.0f);
		uint hard_shadow_packed = asuint(1.0f);
#else
		float4 shadow_value = 1;
		float hard_shadow = 1;
#endif

		// This is the inverse of how large the shadowing window is for the projected sample data. 
		// All values in the LDS sample list are scaled by 1.0 / sample_distance, such that all light directions become parallel.
		// The multiply by sample_distance[0] here is to compensate for the projection divide in the data.
		// The 1.0 / inParameters.SurfaceThickness is to adjust user selected thickness. So a 0.5% thickness will scale depth values from [0,1] to [0,200]. The shadow window is always 1 wide.
		// 1.0 / depth_thickness_scale[0] is because SurfaceThickness is percentage of remaining depth between the sample and the far clip - not a percentage of the full depth range.
		// The min() function is to make sure the window is a minimum width when very close to the light. The +direction term will bias the result so the pixel at the very center of the light is either fully lit or shadowed
		float depth_scale = min(sample_distance[0] + direction, 1.0 / inParameters.SurfaceThickness) * sample_distance[0] / depth_thickness_scale[0];

		start_depth = start_depth * depth_scale - z_sign;
		
	#if TRACK_HIT_UV && !PACKED_HIT_INDEX
		float hit_shadow = 1.0f;
		int hit_index = 0;
		#if DEBUG_HIT_UV
			float2 hit_uv = 0.0f;
		#endif
	#endif

		// The first number of hard shadow samples, a single pixel can produce a full shadow
		[unroll] for (i = 0; i < HARD_SHADOW_SAMPLES; i++)
		{
			float depth_delta = abs(start_depth - DepthData[sample_index + i] * depth_scale);

		#if TRACK_HIT_UV && PACKED_HIT_INDEX
			uint depth_delta_packed = (asuint(depth_delta) & 0xffffff80) | (i + 1);
			hard_shadow_packed = min(hard_shadow_packed, depth_delta_packed);
		#else
			// We want to find the distance of the sample that is closest to the reference depth
			hard_shadow = min(hard_shadow, depth_delta);
		#endif

		#if TRACK_HIT_UV && !PACKED_HIT_INDEX
			if (depth_delta < hit_shadow)
			{
				hit_shadow = depth_delta;
				hit_index = i + 1;
			#if DEBUG_HIT_UV
				hit_uv = UVData[sample_index + i];
			#endif
			}
		#endif
		}

		// Brute force go!
		// The main shadow samples, averaged in to a set of 4 shadow values
		[unroll] for (i = HARD_SHADOW_SAMPLES; i < SAMPLE_COUNT - FADE_OUT_SAMPLES; i++)
		{
			float depth_delta = abs(start_depth - DepthData[sample_index + i] * depth_scale);

		#if TRACK_HIT_UV && PACKED_HIT_INDEX
			uint depth_delta_packed = (asuint(depth_delta) & 0xffffff80) | (i + 1);
			shadow_value_packed[i & 3] = min(shadow_value_packed[i & 3], depth_delta_packed);
		#else
			// Do the same as the hard_shadow code above, but this will accumulate to 4 separate values.
			// By using 4 values, the average shadow can be taken, which can help soften single-pixel shadows.
			shadow_value[i & 3] = min(shadow_value[i & 3], depth_delta);
		#endif

		#if TRACK_HIT_UV && !PACKED_HIT_INDEX
			if (depth_delta < hit_shadow)
			{
				hit_shadow = depth_delta;
				hit_index = i + 1;
			#if DEBUG_HIT_UV
				hit_uv = UVData[sample_index + i];
			#endif
			}
		#endif
		}

		// Final fade out samples
		[unroll] for (i = SAMPLE_COUNT - FADE_OUT_SAMPLES; i < SAMPLE_COUNT; i++)
		{
			// Add the fade value to these samples
			const float fade_out = (float)(i + 1 - (SAMPLE_COUNT - FADE_OUT_SAMPLES)) / (float)(FADE_OUT_SAMPLES + 1) * 0.75;

			float depth_delta = abs(start_depth - DepthData[sample_index + i] * depth_scale) + fade_out;

		#if TRACK_HIT_UV && PACKED_HIT_INDEX
			uint depth_delta_packed = (asuint(depth_delta) & 0xffffff80) | (i + 1);
			shadow_value_packed[i & 3] = min(shadow_value_packed[i & 3], depth_delta_packed);
		#else
			shadow_value[i & 3] = min(shadow_value[i & 3], depth_delta);
		#endif
			
		#if TRACK_HIT_UV && !PACKED_HIT_INDEX
			if (depth_delta < hit_shadow)
			{
				hit_shadow = depth_delta;
				hit_index = i + 1;
			#if DEBUG_HIT_UV
				hit_uv = UVData[sample_index + i];
			#endif
			}
		#endif
		}

	#if TRACK_HIT_UV && PACKED_HIT_INDEX
		uint hit_index = hard_shadow_packed;
		hit_index = min(hit_index, shadow_value_packed.x);
		hit_index = min(hit_index, shadow_value_packed.y);
		hit_index = min(hit_index, shadow_value_packed.z);
		hit_index = min(hit_index, shadow_value_packed.w);
		hit_index &= 0x7f;

		float4 shadow_value = asfloat(shadow_value_packed);
		float hard_shadow = asfloat(hard_shadow_packed);
	#endif

		// Apply the contrast value.
		// A value of 0 indicates a sample was exactly matched to the reference depth (and the result is fully shadowed)
		// We want some boost to this range, so samples don't have to exactly match to produce a full shadow. 
		shadow_value = saturate(shadow_value * (inParameters.ShadowContrast) + (1 - inParameters.ShadowContrast));
		hard_shadow = saturate(hard_shadow * (inParameters.ShadowContrast) + (1 - inParameters.ShadowContrast));

		float result = 0;

		// Take the average of 4 samples, this is useful to reduces aliasing noise in the source depth, especially with long shadows.
		result = dot(shadow_value, 0.25);

		// If the first samples are always producing a hard shadow, then compute this value separately.
		result = min(hard_shadow, result);

		//write the result
		{
			if (inParameters.DebugOutputEdgeMask)
				result = is_edge ? 1 : 0;
			if (inParameters.DebugOutputThreadIndex)
				result = (inGroupThreadID / (float)WAVE_SIZE);
			if (inParameters.DebugOutputWaveIndex)			
				result = frac(inGroupID.x / (float)WAVE_SIZE);

			outResult = result;
			outResultXY = (int2)write_xy;

		#if TRACK_HIT_UV
			float2 hit_xy = floor(start_xy + hit_index * (xy_delta * direction) / WAVE_SIZE);
			#if USE_HALF_PIXEL_OFFSET
				hit_xy += 0.5;
			#endif
				outHitUV = hit_xy * inParameters.InvDepthTextureSize;

			#if DEBUG_HIT_UV
				outHitUV = hit_uv;
			#endif
		#else
			outHitUV = 0.0f;
		#endif
		}
	}

#endif // macro check