// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MeshPaintDilatePixelShader.usf: Mesh texture paint pixel shader
================================================================================*/

#include "Common.ush"

/** BrushRenderTargetTexture - Texture that has our accumulated paint data */
Texture2D<float4> Texture0;
SamplerState Texture0Sampler;

/** SeamMaskRenderTargetTexture - Texture that stores the UV seam mask */
Texture2D<float> Texture1;
SamplerState Texture1Sampler;

/** BrushMaskRenderTargetTexture - Texture that stores mask of where we incrementally painted */
Texture2D<float> Texture2;
SamplerState Texture2Sampler;

/** Width of a pixel in UV space. */
float WidthPixelOffset;
/** Height of a pixel in UV space. */
float HeightPixelOffset;

void Main(
	in float2 InTextureCoordinates : TEXCOORD0,
	in float3 InWorldSpaceVertexPosition : TEXCOORD1,
	in float4 InPosition : SV_POSITION,
	out float4 OutColor : SV_Target0 
	)
{
	// Sample the initial value from the brush texture.
	OutColor = Texture0.Sample( Texture0Sampler, InTextureCoordinates );
	
	// Only need to change the value if the current pixel falls within the white area of the texture seam mask.
	float SeamMask = Texture1.Sample( Texture1Sampler, InTextureCoordinates );
	if ( SeamMask != 0 )
	{
		float2 OffsetConst = float2( WidthPixelOffset, HeightPixelOffset );

		// Arbitrary large number that we will compare to against the squared sample distance.
		// The largest we will hit in 25 samples should be 8 (distance squared = x*x+y*y)
		int LastValidDistance = 10;

		// Sample pixel data from a 5x5 grid around our current pixel.
		float2 OffsetIdx;
		for( OffsetIdx.y = 2; OffsetIdx.y >= -2; OffsetIdx.y-- )
		{
			for( OffsetIdx.x = -2; OffsetIdx.x <= 2; OffsetIdx.x++ )
			{
				// Calculate the squared sample distance from the center.
				int SampleDist = OffsetIdx.x * OffsetIdx.x + OffsetIdx.y * OffsetIdx.y;

				// @todo MeshPaint:If needed, we can improve this by averaging the pixel colors at the same distance
				if ( SampleDist < LastValidDistance )
				{
					// Compute offset coordinates for the pixel and sample the color from the brush mask.
					float2 Coord = InTextureCoordinates + ( OffsetIdx * OffsetConst );

					// A sample is only valid when it falls in the black area of the seam mask and the white area of the brush mask.
					float OffsetSeamMask = Texture1.Sample( Texture1Sampler, Coord );
					float OffsetBrushMask = Texture2.Sample( Texture2Sampler, Coord );

					if ( OffsetSeamMask == 0 && OffsetBrushMask != 0 )
					{
						OutColor = Texture0.Sample( Texture0Sampler, Coord );
						LastValidDistance = SampleDist;
					}
				}
			}
		}
	} 
}
