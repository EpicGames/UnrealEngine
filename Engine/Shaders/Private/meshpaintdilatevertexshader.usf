// Copyright Epic Games, Inc. All Rights Reserved.

/*================================================================================
	MeshPaintDilateVertexShader.usf: Mesh texture paint vertex shader
================================================================================*/

#include "Common.ush"

float4x4 c_Transform;

void Main( in float4 InPosition : ATTRIBUTE0,
		   in float2 InTextureCoordinates : ATTRIBUTE2,
		   in float3 InWorldSpaceVertexPosition : ATTRIBUTE3,
		
		   out float2 OutTextureCoordinates: TEXCOORD0,
		   out float3 OutWorldSpaceVertexPosition : TEXCOORD1,
		   out float4 OutPosition : SV_POSITION )
{
	OutPosition = mul( InPosition, c_Transform );
	OutTextureCoordinates = InTextureCoordinates;
	OutWorldSpaceVertexPosition = InWorldSpaceVertexPosition;
}
